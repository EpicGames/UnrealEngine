// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshHierarchyCmd.h"
#include "Misc/FeedbackContext.h"
#include "Modules/ModuleManager.h"
#include "Animation/Skeleton.h"
#include "Engine/SkeletalMesh.h"
#include "AssetRegistry/AssetData.h"
#include "EdGraph/EdGraphSchema.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetRegistryModule.h"

static FMeshHierarchyCmd MeshHierarchyCmdExec;

bool FMeshHierarchyCmd::Exec_Editor(UWorld*, const TCHAR* Cmd, FOutputDevice& Ar)
{
	bool bResult = false;
	if (FParse::Command(&Cmd, TEXT("TMH")))
	{
		Ar.Log(TEXT("Starting Mesh Test"));
		FARFilter Filter;
		Filter.ClassPaths.Add(USkeletalMesh::StaticClass()->GetClassPathName());

		TArray<FAssetData> SkeletalMeshes;
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		AssetRegistryModule.Get().GetAssets(Filter, SkeletalMeshes);

		const FText StatusUpdate = NSLOCTEXT("MeshHierarchyCmd", "RemoveUnusedBones_ProcessingAssets", "Processing Skeletal Meshes");
		GWarn->BeginSlowTask(StatusUpdate, true);

		// go through all assets try load
		for (int32 MeshIdx = 0; MeshIdx < SkeletalMeshes.Num(); ++MeshIdx)
		{
			GWarn->StatusUpdate(MeshIdx, SkeletalMeshes.Num(), StatusUpdate);

			USkeletalMesh* Mesh = Cast<USkeletalMesh>(SkeletalMeshes[MeshIdx].GetAsset());
			if (Mesh->GetSkeleton())
			{
				FName MeshRoot = Mesh->GetRefSkeleton().GetBoneName(0);
				FName SkelRoot = Mesh->GetSkeleton()->GetReferenceSkeleton().GetBoneName(0);

				if (MeshRoot != SkelRoot)
				{
					Ar.Logf(TEXT("Mesh Found '%s' %s->%s"), *Mesh->GetName(), *MeshRoot.ToString(), *SkelRoot.ToString());
				}
			}
		}

		GWarn->EndSlowTask();
		Ar.Log(TEXT("Mesh Test Finished"));
	}
	return bResult;
}
