// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/CoreMisc.h"

class FMeshHierarchyCmd : private FSelfRegisteringExec
{
protected:
	/** Console commands, see embeded usage statement **/
	virtual bool Exec_Editor(UWorld*, const TCHAR* Cmd, FOutputDevice& Ar) override;
};
