// Copyright Epic Games, Inc. All Rights Reserved.

#include "SControlRigMappingWindow.h"
#include "Misc/MessageDialog.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Styling/AppStyle.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SToolTip.h"
#include "IDocumentation.h"
#include "ScopedTransaction.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "Engine/Blueprint.h"
#include "Engine/SkeletalMesh.h"
#include "Animation/NodeMappingProviderInterface.h"
#include "AnimationRuntime.h"

#define LOCTEXT_NAMESPACE "SControlRigMappingWindow"


//////////////////////////////////////////////////////////////////////////
// SAnimationMappingWindow

void SControlRigMappingWindow::Construct(const FArguments& InArgs, const TWeakObjectPtr<class USkeletalMesh>& InEditableMesh, FSimpleMulticastDelegate& InOnPostUndo)
{
	EditableSkeletalMeshPtr = InEditableMesh;
	InOnPostUndo.Add(FSimpleDelegate::CreateSP(this, &SControlRigMappingWindow::PostUndo));

	const FString DocLink = TEXT("Shared/Editors/Persona");
	ChildSlot
	[
		SNew (SVerticalBox)

		+ SVerticalBox::Slot()
		.Padding(5, 5)
		.AutoHeight()
		[
			// explain this is Control Rig  window
			// and what it is
			SNew(STextBlock)
			.TextStyle( FAppStyle::Get(), "Persona.RetargetManager.ImportantText" )
			.Text(LOCTEXT("ControlRigMapping_Title", "Configure Control Rig Settings"))
		]

		+ SVerticalBox::Slot()
		.Padding(5, 5)
		.AutoHeight()
		[
			// explainint this is Control Rig  window
			// and what it is
			SNew(STextBlock)
			.AutoWrapText(true)
			.ToolTip(IDocumentation::Get()->CreateToolTip(LOCTEXT("ControlRigMappingTooltip", "Add new Control Rig, and remap or edit mapping information."),
																			NULL,
																			DocLink,
																			TEXT("NodeMapping")))
			.Font(FAppStyle::GetFontStyle(TEXT("Persona.RetargetManager.FilterFont")))
			.Text(LOCTEXT("ControlRigMappingDescription", "You can add/delete Control Rig Mapping Configuration."))
		]

		// select and combo box
		+ SVerticalBox::Slot()
		.Padding(5)
		.AutoHeight()
		[
			SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
			.Padding(2)
			.AutoWidth()
			[
				// is this safe with uobject? - i.e. if this gets GC-ed?
				SAssignNew(MappingOptionBox, SComboBox< TSharedPtr<class UNodeMappingContainer*> >)
				.ContentPadding(FMargin(6.0f, 2.0f))
				.OptionsSource(&MappingOptionBoxList)
				.OnGenerateWidget(this, &SControlRigMappingWindow::HandleMappingOptionBoxGenerateWidget)
				.OnSelectionChanged(this, &SControlRigMappingWindow::HandleMappingOptionBoxSelectionChanged)
				[
					SNew(STextBlock)
					.Text(this, &SControlRigMappingWindow::HandleMappingOptionBoxContentText)
					.Font(FAppStyle::GetFontStyle(TEXT("Persona.RetargetManager.FilterFont")))
				]
			]

			// add new or delete button
			+SHorizontalBox::Slot()
			.Padding(2)
			.AutoWidth()
			[
				SNew(SButton)
				.OnClicked(FOnClicked::CreateSP(this, &SControlRigMappingWindow::OnAddNodeMappingButtonClicked))
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Text(LOCTEXT("AddNodeMappingButton_Label", "Add New"))
			]

			+SHorizontalBox::Slot()
			.Padding(2)
			.AutoWidth()
			[
				SNew(SButton)
				.OnClicked(FOnClicked::CreateSP(this, &SControlRigMappingWindow::OnDeleteNodeMappingButtonClicked))
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Text(LOCTEXT("DeleteNodeMappingButton_Label", "Delete Current"))
			]

			+SHorizontalBox::Slot()
			.Padding(2)
			.AutoWidth()
			[
				SNew(SButton)
				.OnClicked(FOnClicked::CreateSP(this, &SControlRigMappingWindow::OnRefreshNodeMappingButtonClicked))
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Text(LOCTEXT("RefreshNodeMappingButton_Label", "Refresh Mapping"))
			]
		]

		+SVerticalBox::Slot()
		.Padding(2, 5)
		.AutoHeight()
		[
			SNew(SBox)
			.MaxDesiredHeight(500)
			.Content()
			[
				// add bone mapper window
				SAssignNew(BoneMappingWidget, SBoneMappingBase, InOnPostUndo)
				.OnBoneMappingChanged(this, &SControlRigMappingWindow::OnBoneMappingChanged)
				.OnGetBoneMapping(this, &SControlRigMappingWindow::GetBoneMapping)
				.OnCreateBoneMapping(this, &SControlRigMappingWindow::CreateBoneMappingList)
				.OnGetReferenceSkeleton(this, &SControlRigMappingWindow::GetReferenceSkeleton)
			]
		]

		+SVerticalBox::Slot()
		.Padding(2, 5)
		.AutoHeight()
		[
			SNew(SSeparator)
			.Orientation(Orient_Horizontal)
		]
	];

	RefreshList();
}

void SControlRigMappingWindow::PostUndo()
{
	RefreshList();
}

TSharedRef<SWidget> SControlRigMappingWindow::HandleMappingOptionBoxGenerateWidget(TSharedPtr<class UNodeMappingContainer*> Item)
{
	// @todo: create tooltip with path
	return SNew(STextBlock)
		.Text(FText::FromString(*(*Item)->GetDisplayName()))
		.Font(FAppStyle::GetFontStyle(TEXT("Persona.RetargetManager.FilterFont")));
}

void SControlRigMappingWindow::HandleMappingOptionBoxSelectionChanged(TSharedPtr<class UNodeMappingContainer*> Item, ESelectInfo::Type SelectInfo)
{
	for (int32 Index=0; Index <MappingOptionBoxList.Num(); ++Index)
	{
		if (MappingOptionBoxList[Index] == Item)
		{
			CurrentlySelectedIndex = Index;
			break;
		}
	}

	// @todo: refresh mapping window
	MappingOptionBox->RefreshOptions();
	BoneMappingWidget->RefreshBoneMappingList();
}

FText SControlRigMappingWindow::HandleMappingOptionBoxContentText() const
{
	if (MappingOptionBoxList.IsValidIndex(CurrentlySelectedIndex))
	{
		return FText::FromString(*(*MappingOptionBoxList[CurrentlySelectedIndex])->GetDisplayName());
	}

	return LOCTEXT("ControlRigMappingWindow_NoneSelected", "None Selected. Create New.");
}

void SControlRigMappingWindow::AddNodeMapping(UBlueprint* NewSourceControlRig)
{
	if ( ensureAlways(EditableSkeletalMeshPtr.IsValid()) )
	{
		// make sure it doesn't have it yet. If so, we warn them. 
		// @todo: It is possible we could support multi, but then we should give unique display name, and use that to identify. 
		// this all can get very messy, so for now, we just support one for each
		// create new mapper object
		USkeletalMesh* SkeletalMesh = EditableSkeletalMeshPtr.Get();
		TArray<TObjectPtr<UNodeMappingContainer>>& NodeMappingData = SkeletalMesh->GetNodeMappingData();
		for (int32 Index = 0; Index < NodeMappingData.Num(); ++Index)
		{
			if (NewSourceControlRig == NodeMappingData[Index]->GetSourceAsset())
			{
				FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("ControlRigConfigAlreadyExists", "The same Control Rig configuration already exists in this mesh. Edit current existing setting."));
				return;
			}
		}

		{
			const FScopedTransaction Transaction(LOCTEXT("ControlRigMapping_AddNew", "Add New Mapping"));

			SkeletalMesh->Modify();
			UNodeMappingContainer* NewMapperObject = NewObject<UNodeMappingContainer>(SkeletalMesh);
			NewMapperObject->SetSourceAsset(NewSourceControlRig);
			// set target asset as skeletalmesh
			NewMapperObject->SetTargetAsset(SkeletalMesh);
			// add default mapping, this will map default settings
			NewMapperObject->AddDefaultMapping();
			CurrentlySelectedIndex = SkeletalMesh->GetNodeMappingData().Add(NewMapperObject);

			RefreshList();
		}
	}
}

FReply SControlRigMappingWindow::OnDeleteNodeMappingButtonClicked()
{
	const FScopedTransaction Transaction(LOCTEXT("ControlRigMapping_Delete", "Delete Selected Mapping"));

	// create new mapper object
	USkeletalMesh* SkeletalMesh = EditableSkeletalMeshPtr.Get();
	TArray<TObjectPtr<UNodeMappingContainer>>& NodeMappingData = SkeletalMesh->GetNodeMappingData();
	if (NodeMappingData.IsValidIndex(CurrentlySelectedIndex))
	{
		SkeletalMesh->Modify();
		NodeMappingData.RemoveAt(CurrentlySelectedIndex);
		CurrentlySelectedIndex = (NodeMappingData.Num() > 0) ? 0 : INDEX_NONE;
		RefreshList();
	}

	return FReply::Handled();
}

FReply SControlRigMappingWindow::OnRefreshNodeMappingButtonClicked()
{
	// create new mapper object
	USkeletalMesh* SkeletalMesh = EditableSkeletalMeshPtr.Get();
	if (SkeletalMesh->GetNodeMappingData().IsValidIndex(CurrentlySelectedIndex))
	{
		const FScopedTransaction Transaction(LOCTEXT("ControlRigMapping_Refresh", "Refresh Node Mapping"));

		UNodeMappingContainer* Container = GetCurrentBoneMappingContainer();
		if (SkeletalMesh && Container)
		{
			SkeletalMesh->Modify();
			Container->RefreshDataFromAssets();
		}
	}

	return FReply::Handled();
}
void SControlRigMappingWindow::OnAddNodeMapping()
{
	// show list of skeletalmeshes that they can choose from
	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

	FAssetPickerConfig AssetPickerConfig;
 	AssetPickerConfig.Filter.ClassPaths.Add(UBlueprint::StaticClass()->GetClassPathName());
 	AssetPickerConfig.Filter.bRecursiveClasses = true;
	AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateSP(this, &SControlRigMappingWindow::OnAssetSelectedFromMeshPicker);
	AssetPickerConfig.bAllowNullSelection = false;
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::Tile;
	AssetPickerConfig.OnShouldFilterAsset = FOnShouldFilterAsset::CreateSP(this, &SControlRigMappingWindow::OnShouldFilterAnimAsset);

	TSharedRef<SWidget> Widget = SNew(SBox)
		.WidthOverride(384)
		.HeightOverride(768)
		[
			SNew(SBorder)
			.BorderBackgroundColor(FLinearColor(0.25f, 0.25f, 0.25f, 1.f))
			.Padding(2)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.Padding(8)
			[
				ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
			]
		]
		];

	FSlateApplication::Get().PushMenu(
		AsShared(),
		FWidgetPath(),
		Widget,
		FSlateApplication::Get().GetCursorPos(),
		FPopupTransitionEffect(FPopupTransitionEffect::TopMenu)
	);
}

FReply SControlRigMappingWindow::OnAddNodeMappingButtonClicked()
{
	OnAddNodeMapping();
	return FReply::Handled();
}

void SControlRigMappingWindow::OnAssetSelectedFromMeshPicker(const FAssetData& AssetData)
{
	// add to mapping now
	AddNodeMapping(Cast<UBlueprint>(AssetData.GetAsset()));
	FSlateApplication::Get().DismissAllMenus();
}

bool SControlRigMappingWindow::OnShouldFilterAnimAsset(const FAssetData& AssetData) const
{
	FString ParentClassName;
	if (AssetData.GetTagValue(FBlueprintTags::NativeParentClassPath, ParentClassName))
	{
		if (ParentClassName.IsEmpty() == false)
		{
			UClass* ParentClass = UClass::TryFindTypeSlow<UClass>(FPackageName::ExportTextPathToObjectPath(ParentClassName));
			while (ParentClass && ParentClass != UObject::StaticClass())
			{
				if (ParentClass->ImplementsInterface(UNodeMappingProviderInterface::StaticClass()))
				{
					return false;
				}

				ParentClass = ParentClass->GetSuperClass();
			}
		}
	}

	return true;
}

void SControlRigMappingWindow::RefreshList()
{
	// update list of mapping options
	// @todo: have to make sure there is no duplicated name. If so, we'll have to create fake name
	// for now, do path name
	MappingOptionBoxList.Empty();
	TArray<TObjectPtr<UNodeMappingContainer>>& NodeMappingData = EditableSkeletalMeshPtr->GetNodeMappingData();
	for (int32 Index = 0; Index < NodeMappingData.Num(); ++Index)
	{
		class UNodeMappingContainer* MappingData = NodeMappingData[Index];
		if (MappingData)
		{
			MappingOptionBoxList.Add(MakeShareable(new UNodeMappingContainer*(MappingData)));
		}
		else
		{
			// we have to add all index because we can't skip it
			ensureAlwaysMsgf(false, TEXT("Invalid node mapping data exists. This will cause issue with later indices."));
		}
	}

	CurrentlySelectedIndex = (MappingOptionBoxList.Num() > 0) ? 0 : INDEX_NONE;

	MappingOptionBox->RefreshOptions();

	BoneMappingWidget->RefreshBoneMappingList();
}

class UNodeMappingContainer* SControlRigMappingWindow::GetCurrentBoneMappingContainer() const
{
	if (EditableSkeletalMeshPtr.IsValid())
	{
		USkeletalMesh* Mesh = EditableSkeletalMeshPtr.Get();
		TArray<TObjectPtr<UNodeMappingContainer>>& NodeMappingData = Mesh->GetNodeMappingData();
		if (NodeMappingData.IsValidIndex(CurrentlySelectedIndex))
		{
			return NodeMappingData[CurrentlySelectedIndex];
		}
	}

	return nullptr;
}

void SControlRigMappingWindow::OnBoneMappingChanged(FName NodeName, FName BoneName)
{
	// set mapping
	USkeletalMesh* Mesh = EditableSkeletalMeshPtr.Get();
	UNodeMappingContainer* Container = GetCurrentBoneMappingContainer();
	if (Mesh && Container)
	{
		Container->AddMapping(NodeName, BoneName);
	}
}

FName SControlRigMappingWindow::GetBoneMapping(FName NodeName)
{
	UNodeMappingContainer* Container = GetCurrentBoneMappingContainer();
	if (Container)
	{
		const FName* Target = Container->GetNodeMappingTable().Find(NodeName);;
		return Target? (*Target) : NAME_None;
	}

	return NAME_None;
}

void SControlRigMappingWindow::CreateBoneMappingList(const FString& SearchText, TArray< TSharedPtr<FDisplayedBoneMappingInfo> >& BoneMappingList)
{
	BoneMappingList.Empty();

	const FReferenceSkeleton RefSkeleton = EditableSkeletalMeshPtr.Get()->GetRefSkeleton();
	UNodeMappingContainer* Container = GetCurrentBoneMappingContainer();
	if (Container)
	{
		bool bDoFiltering = !SearchText.IsEmpty();
		const TMap<FName, FNodeItem>& SourceItems = Container->GetSourceItems();
		const TMap<FName, FName>& SourceToTarget = Container->GetNodeMappingTable();

		if ( SourceItems.Num() > 0 )
		{
			for ( auto Iter = SourceItems.CreateConstIterator() ; Iter; ++Iter )
			{
				const FName& Name = Iter.Key();
				const FString& DisplayName = Name.ToString();
				const FName* BoneName = SourceToTarget.Find(Name);

				if (bDoFiltering)
				{
					// make sure it doesn't fit any of them
					if (!DisplayName.Contains(SearchText) && (BoneName && !(*BoneName).ToString().Contains(SearchText)))
					{
						continue; // Skip items that don't match our filter
					}
				}

				TSharedRef<FDisplayedBoneMappingInfo> Info = FDisplayedBoneMappingInfo::Make(Name, DisplayName);
				BoneMappingList.Add(Info);
			}
		}
	}
}

const struct FReferenceSkeleton& SControlRigMappingWindow::GetReferenceSkeleton() const
{
	static FReferenceSkeleton DummySkeleton;
	return (EditableSkeletalMeshPtr.IsValid())? EditableSkeletalMeshPtr.Get()->GetRefSkeleton() : DummySkeleton;
}
#undef LOCTEXT_NAMESPACE

