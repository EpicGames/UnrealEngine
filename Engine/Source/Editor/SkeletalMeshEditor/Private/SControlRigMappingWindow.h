// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
//#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/SComboBox.h"
#include "Animation/NodeMappingContainer.h"
#include "AssetRegistry/AssetData.h"
#include "SBoneMappingBase.h"

//////////////////////////////////////////////////////////////////////////
// SControlRigMappingWindow

class SControlRigMappingWindow : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SControlRigMappingWindow )
	{}
	SLATE_END_ARGS()

	/**
	* Slate construction function
	*
	* @param InArgs - Arguments passed from Slate
	*
	*/
	void Construct(const FArguments& InArgs, const TWeakObjectPtr<class USkeletalMesh>& InEditableMesh, FSimpleMulticastDelegate& InOnPostUndo);

private:
	/** The editable skeletalmesh */
	TWeakObjectPtr<class USkeletalMesh> EditableSkeletalMeshPtr;

	/** Combo box for list of mapping selection */
	TSharedPtr<SComboBox< TSharedPtr<class UNodeMappingContainer*> >>		MappingOptionBox;
	TArray< TSharedPtr<class UNodeMappingContainer*> >						MappingOptionBoxList;

	/** Currently selected index */
	int32 CurrentlySelectedIndex;

	/** Bone mapping widget */
	TSharedPtr<SBoneMappingBase> BoneMappingWidget;

	/** Delegate for undo/redo transaction **/
	void PostUndo();

	/** UI Handler for drop down combo box */
	TSharedRef<SWidget> HandleMappingOptionBoxGenerateWidget(TSharedPtr<class UNodeMappingContainer*> Item);
	void HandleMappingOptionBoxSelectionChanged(TSharedPtr<class UNodeMappingContainer*> Item, ESelectInfo::Type SelectInfo);
	FText HandleMappingOptionBoxContentText() const;

	/** Button handler **/
	FReply OnAddNodeMappingButtonClicked();
	FReply OnDeleteNodeMappingButtonClicked();
	FReply OnRefreshNodeMappingButtonClicked();
	/**
	* Handler for adding new retarget source. It displays the asset picker
	*/
	void OnAddNodeMapping();
	/**
	*
	*/
	void AddNodeMapping(UBlueprint* NewSourceController);
	void OnAssetSelectedFromMeshPicker(const FAssetData& AssetData);
	/** Delegate to handle filtering of asset pickers */
	bool OnShouldFilterAnimAsset(const FAssetData& AssetData) const;
	/** refresh drop down list */
	void RefreshList();

	// bone mapping interface
	void OnBoneMappingChanged(FName NodeName, FName BoneName);
	FName GetBoneMapping(FName NodeName);
	const struct FReferenceSkeleton& GetReferenceSkeleton() const;
	void CreateBoneMappingList(const FString& SearchText, TArray< TSharedPtr<FDisplayedBoneMappingInfo> >& BoneMappingList);

	class UNodeMappingContainer* GetCurrentBoneMappingContainer() const;
};
