// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletalMeshClothingSystemUtilities.h"

#if WITH_EDITOR
#include "ClothingAssetBase.h"
#include "Engine/SkeletalMesh.h"
#include "Rendering/SkeletalMeshLODModel.h"
#include "Rendering/SkeletalMeshModel.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "FSkeletalMeshClothingSystemUtilities"

bool FSkeletalMeshClothingSystemUtilities::AssignClothingToSection(USkeletalMesh* SkeletalMesh, UClothingAssetBase* ClothingAsset, int32 SkeletalMeshLodIndex, int32 SkeletalMeshSection, int32 ClothingLodIndex, FString* OutError)
{
	if (!SkeletalMesh || !SkeletalMesh->GetImportedModel())
	{
		if (OutError) { *OutError = TEXT("SkeletalMesh or its imported model is null."); }
		return false;
	}

	if (!ClothingAsset)
	{
		if (OutError) { *OutError = TEXT("ClothingAsset is null."); }
		return false;
	}

	if (!SkeletalMesh->GetImportedModel()->LODModels.IsValidIndex(SkeletalMeshLodIndex))
	{
		if (OutError) { *OutError = FString::Printf(TEXT("LOD index %d is out of range (mesh has %d LODs)."), SkeletalMeshLodIndex, SkeletalMesh->GetImportedModel()->LODModels.Num()); }
		return false;
	}

	FSkeletalMeshLODModel& LODModel = SkeletalMesh->GetImportedModel()->LODModels[SkeletalMeshLodIndex];
	if (!LODModel.Sections.IsValidIndex(SkeletalMeshSection))
	{
		if (OutError) { *OutError = FString::Printf(TEXT("Section index %d is out of range (LOD %d has %d sections)."), SkeletalMeshSection, SkeletalMeshLodIndex, LODModel.Sections.Num()); }
		return false;
	}

	const FSkelMeshSection& Section = LODModel.Sections[SkeletalMeshSection];

	FScopedSkeletalMeshPostEditChange ScopedPostEditChange(SkeletalMesh);
	FScopedTransaction Transaction(
		LOCTEXT("AssignClothing", "Assign Section Cloth"));
	SkeletalMesh->Modify();
	ClothingAsset->Modify();

	FSkelMeshSourceSectionUserData& OriginalSectionData =
		LODModel.UserSectionsData.FindOrAdd(Section.OriginalDataSectionIndex);

	auto ClearOriginalSectionUserData = [&OriginalSectionData]()
		{
			OriginalSectionData.CorrespondClothAssetIndex = INDEX_NONE;
			OriginalSectionData.ClothingData.AssetGuid = FGuid();
			OriginalSectionData.ClothingData.AssetLodIndex = INDEX_NONE;
		};

	if (UClothingAssetBase* CurrentAsset = SkeletalMesh->GetSectionClothingAsset(SkeletalMeshLodIndex, SkeletalMeshSection))
	{
		CurrentAsset->Modify();
		CurrentAsset->UnbindFromSkeletalMesh(SkeletalMesh, SkeletalMeshLodIndex, SkeletalMeshSection);
		ClearOriginalSectionUserData();
	}

	if (ClothingAsset->BindToSkeletalMesh(SkeletalMesh, SkeletalMeshLodIndex, SkeletalMeshSection, ClothingLodIndex))
	{
		int32 AssetIndex = INDEX_NONE;
		verify(SkeletalMesh->GetMeshClothingAssets().Find(ClothingAsset, AssetIndex));
		OriginalSectionData.CorrespondClothAssetIndex = static_cast<int16>(AssetIndex);
		OriginalSectionData.ClothingData.AssetGuid = ClothingAsset->GetAssetGuid();
		OriginalSectionData.ClothingData.AssetLodIndex = ClothingLodIndex;
		return true;
	}

	if (OutError) { *OutError = FString::Printf(TEXT("Failed to bind clothing asset '%s' to LOD %d section %d (clothing LOD %d may be invalid or incompatible)."), *ClothingAsset->GetName(), SkeletalMeshLodIndex, SkeletalMeshSection, ClothingLodIndex); }
	return false;
}

bool FSkeletalMeshClothingSystemUtilities::RemoveClothingFromSection(USkeletalMesh* SkeletalMesh, int32 LodIndex, int32 SectionIndex, FString* OutError)
{
	if (!SkeletalMesh || !SkeletalMesh->GetImportedModel())
	{
		if (OutError) { *OutError = TEXT("SkeletalMesh or its imported model is null."); }
		return false;
	}

	if (!SkeletalMesh->GetImportedModel()->LODModels.IsValidIndex(LodIndex))
	{
		if (OutError) { *OutError = FString::Printf(TEXT("LOD index %d is out of range (mesh has %d LODs)."), LodIndex, SkeletalMesh->GetImportedModel()->LODModels.Num()); }
		return false;
	}

	FSkeletalMeshLODModel& LODModel = SkeletalMesh->GetImportedModel()->LODModels[LodIndex];
	if (!LODModel.Sections.IsValidIndex(SectionIndex))
	{
		if (OutError) { *OutError = FString::Printf(TEXT("Section index %d is out of range (LOD %d has %d sections)."), SectionIndex, LodIndex, LODModel.Sections.Num()); }
		return false;
	}

	const FSkelMeshSection& Section = LODModel.Sections[SectionIndex];

	FScopedSkeletalMeshPostEditChange ScopedPostEditChange(SkeletalMesh);
	FScopedTransaction Transaction(
		LOCTEXT("RemoveClothing", "Remove Cloth from Section"));
	SkeletalMesh->Modify();

	FSkelMeshSourceSectionUserData& OriginalSectionData =
		LODModel.UserSectionsData.FindOrAdd(Section.OriginalDataSectionIndex);

	if (UClothingAssetBase* CurrentAsset = SkeletalMesh->GetSectionClothingAsset(LodIndex, SectionIndex))
	{
		CurrentAsset->Modify();
		CurrentAsset->UnbindFromSkeletalMesh(SkeletalMesh, LodIndex, SectionIndex);
		OriginalSectionData.CorrespondClothAssetIndex = INDEX_NONE;
		OriginalSectionData.ClothingData.AssetGuid = FGuid();
		OriginalSectionData.ClothingData.AssetLodIndex = INDEX_NONE;
		return true;
	}
	return false;
}


#undef LOCTEXT_NAMESPACE
#endif // WITH_EDITOR