// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletalMeshEditor.h"

#include "AnimationEditorViewportClient.h"
#include "SkeletalMeshEditorCommands.h"
#include "SkeletalMeshEditorMode.h"

#include "Algo/Transform.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "AssetRegistry/AssetData.h"
#include "Async/Async.h"
#include "ClothingAsset.h"
#include "ClothingSystemEditorInterfaceModule.h"
#include "ComponentReregisterContext.h"
#include "EdGraph/EdGraphSchema.h"
#include "Editor/EditorEngine.h"
#include "EditorFramework/AssetImportData.h"
#include "EditorModeManager.h"
#include "EditorReimportHandler.h"
#include "EditorViewportClient.h"
#include "Engine/SkinnedAssetAsyncCompileUtils.h"
#include "EngineGlobals.h"
#include "EngineUtils.h"
#include "Factories/FbxSkeletalMeshImportData.h"
#include "FbxMeshUtils.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IAssetFamily.h"
#include "IDetailsView.h"
#include "IPersonaPreviewScene.h"
#include "IPersonaToolkit.h"
#include "IPersonaViewport.h"
#include "ISkeletalMeshEditorModule.h"
#include "ISkeletonEditorModule.h"
#include "ISkeletonTree.h"
#include "ISkeletonTreeItem.h"
#include "LODUtilities.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "PersonaCommonCommands.h"
#include "PersonaModule.h"
#include "PersonaToolMenuContext.h"
#include "Preferences/PersonaOptions.h"
#include "Rendering/SkeletalMeshModel.h"
#include "ScopedTransaction.h"
#include "SCreateClothingSettingsPanel.h"
#include "Settings/EditorExperimentalSettings.h"
#include "SkeletalMeshToolMenuContext.h"
#include "ToolMenus.h"
#include "Engine/SkeletalMeshLODSettings.h"
#include "Engine/SkeletalMeshSocket.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Misc/CoreMisc.h"
#include "Toolkits/AssetEditorToolkitMenuContext.h"
#include "MeshMergeModule.h"
#include "AssetToolsModule.h"
#include "DetailLayoutBuilder.h"
#include "IAssetTools.h"
#include "IMorphTargetViewer.h"
#include "SkeletalMeshEditorContextMenuContext.h"
#include "SkeletalMeshClothingSystemUtilities.h"
#include "Styling/AppStyle.h"
#include "InterchangeAssetImportData.h"
#include "InterchangeManager.h"
#include "SkeletalMeshEditorModeUILayer.h"
#include "EditorDragTools/FrustumSelectInteraction.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "ViewportInteractions/ViewportInteractionsBehaviorSource.h"
#include "ViewportInteractions/ViewportInteractionsUtilities.h"
#include "ViewportInteractions/ViewportOrbitInteraction.h"

const FName SkeletalMeshEditorAppIdentifier = FName(TEXT("SkeletalMeshEditorApp"));

const FName SkeletalMeshEditorModes::SkeletalMeshEditorMode(TEXT("SkeletalMeshEditorMode"));

const FName SkeletalMeshEditorTabs::DetailsTab(TEXT("DetailsTab"));
const FName SkeletalMeshEditorTabs::SkeletonTreeTab(TEXT("SkeletonTreeView"));
const FName SkeletalMeshEditorTabs::AssetDetailsTab(TEXT("AnimAssetPropertiesTab"));
const FName SkeletalMeshEditorTabs::ViewportTab(TEXT("Viewport"));
const FName SkeletalMeshEditorTabs::AdvancedPreviewTab(TEXT("AdvancedPreviewTab"));
const FName SkeletalMeshEditorTabs::MorphTargetsTab("MorphTargetsTab");
const FName SkeletalMeshEditorTabs::CurveMetadataTab(TEXT("AnimCurveMetadataEditorTab"));
const FName SkeletalMeshEditorTabs::FindReplaceTab("FindReplaceTab");


static const FName ViewportMenuId("SkeletalMeshEditor.MeshContextMenu");


DEFINE_LOG_CATEGORY(LogSkeletalMeshEditor);

#define LOCTEXT_NAMESPACE "SkeletalMeshEditor"

FSkeletalMeshEditor::FSkeletalMeshEditor()
{
	UEditorEngine* Editor = Cast<UEditorEngine>(GEngine);
	if (Editor != nullptr)
	{
		Editor->RegisterForUndo(this);
	}
}

FSkeletalMeshEditor::~FSkeletalMeshEditor()
{
	if (FReimportManager::Instance())
	{
		FReimportManager::Instance()->OnPostReimport().RemoveAll(this);
	}

	UEditorEngine* Editor = Cast<UEditorEngine>(GEngine);
	if (Editor != nullptr)
	{
		Editor->UnregisterForUndo(this);
	}

	//We have to do this differently, make sure we do not have any cloth paint mode active before deleting the PersonaToolkit
	{
		//At this point the ClothPaintMode, if exist is deactivate but not destroy
		//This is why we do not verify if the mode is active. Calling DestroyMode on a unexisting mode is ok
		const FEditorModeID ClothModeID = FName("ClothPaintMode");
		GetEditorModeManager().DestroyMode(ClothModeID);
	}
	
	// Reset the preview scene mesh before closing the toolkit or destroying the preview scene so the viewports can clean up properly.
	// This is due to the viewports directly needing to use delegates on a nested USkeletalMesh parented to a debug component the editor uses.
	// The USkeletalMesh persists beyond the lifetime of the debug component used by the toolkit or viewports,
	// which can cause issues for viewports with delegates for on change notifications on the skeletal mesh to track undo/redo for things like floor mesh movement,
	// since the floor mesh position is stored on the skeletal mesh, and not the debug component.
	if (PersonaToolkit.IsValid())
	{
		constexpr bool bSetPreviewMeshInAsset = false;
		PersonaToolkit->SetPreviewMesh(nullptr, bSetPreviewMeshInAsset);
	}
}

bool IsReductionParentBaseLODUseSkeletalMeshBuildWorkflow(USkeletalMesh* SkeletalMesh, int32 TestLODIndex)
{
	FSkeletalMeshLODInfo* LODInfo = SkeletalMesh->GetLODInfo(TestLODIndex);
	if (LODInfo == nullptr || !SkeletalMesh->GetImportedModel() || !SkeletalMesh->GetImportedModel()->LODModels.IsValidIndex(TestLODIndex))
	{
		return false;
	}
	if (SkeletalMesh->HasMeshDescription(TestLODIndex))
	{
		return true;
	}
	if (LODInfo->bHasBeenSimplified || SkeletalMesh->IsReductionActive(TestLODIndex))
	{
		int32 ReduceBaseLOD = LODInfo->ReductionSettings.BaseLOD;
		if (ReduceBaseLOD < TestLODIndex)
		{
			return IsReductionParentBaseLODUseSkeletalMeshBuildWorkflow(SkeletalMesh, ReduceBaseLOD);
		}
	}
	return false;
}

bool FSkeletalMeshEditor::OnRequestClose(EAssetEditorCloseReason InCloseReason)
{
	bool bAllowClose = GetEditorModeManager().OnRequestClose();
	if (!bAllowClose)
	{
		return false;
	}

	if (PersonaToolkit.IsValid() && SkeletalMesh)
	{
		bool bHaveModifiedLOD = false;
		for (int32 LODIndex = 0; LODIndex < SkeletalMesh->GetLODNum(); ++LODIndex)
		{
			FSkeletalMeshLODInfo* LODInfo = SkeletalMesh->GetLODInfo(LODIndex);
			if (LODInfo == nullptr || !SkeletalMesh->GetImportedModel() || !SkeletalMesh->GetImportedModel()->LODModels.IsValidIndex(LODIndex))
			{
				continue;
			}
			
			bool bValidLODSettings = false;
			if (SkeletalMesh->GetLODSettings() != nullptr)
			{
				const int32 NumSettings = FMath::Min(SkeletalMesh->GetLODSettings()->GetNumberOfSettings(), SkeletalMesh->GetLODNum());
				if (LODIndex < NumSettings)
				{
					bValidLODSettings = true;
				}
			}
			const FSkeletalMeshLODGroupSettings* SkeletalMeshLODGroupSettings = bValidLODSettings ? &SkeletalMesh->GetLODSettings()->GetSettingsForLODLevel(LODIndex) : nullptr;

			FGuid BuildGUID = LODInfo->ComputeDeriveDataCacheKey(SkeletalMeshLODGroupSettings);
			if (LODInfo->BuildGUID != BuildGUID)
			{
				bHaveModifiedLOD = true;
				break;
			}
			FString BuildStringID = SkeletalMesh->GetImportedModel()->LODModels[LODIndex].GetLODModelDeriveDataKey();
			if (SkeletalMesh->GetImportedModel()->LODModels[LODIndex].BuildStringID != BuildStringID)
			{
				bHaveModifiedLOD = true;
				break;
			}
		}

		if (bHaveModifiedLOD && InCloseReason != EAssetEditorCloseReason::AssetForceDeleted)
		{
			// find out what the user wants to do with this dirty material
			EAppReturnType::Type OkCancelReply = FMessageDialog::Open(
				EAppMsgType::OkCancel,
				FText::Format(LOCTEXT("SkeletalMeshEditorShouldApplyLODChanges", "We have to apply level of detail changes to {0} before exiting the skeletal mesh editor."), FText::FromString(PersonaToolkit->GetMesh()->GetName()))
			);

			switch (OkCancelReply)
			{
			case EAppReturnType::Ok:
				{
					SkeletalMesh->MarkPackageDirty();
					SkeletalMesh->PostEditChange();
					bAllowClose = true;
				}
				break;
			case EAppReturnType::Cancel:
				// Don't exit.
				bAllowClose = false;
				break;
			}
		}
	}

	return bAllowClose;
}

void FSkeletalMeshEditor::SaveAsset_Execute()
{
	OnPreSaveAssetDelegate.Broadcast();
	ISkeletalMeshEditor::SaveAsset_Execute();
}

void FSkeletalMeshEditor::SaveAssetAs_Execute()
{
	OnPreSaveAssetAsDelegate.Broadcast();
	ISkeletalMeshEditor::SaveAssetAs_Execute();
}

void FSkeletalMeshEditor::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	SkeletalMeshEditorMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_SkeletalMeshEditor", "SkeletalMeshEditor"));
}

void FSkeletalMeshEditor::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);
}

void FSkeletalMeshEditor::InitSkeletalMeshEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, USkeletalMesh* InSkeletalMesh)
{
	SkeletalMesh = InSkeletalMesh;

	// Catch re-imports triggered outside this editor (Content Browser, scripted, etc.).
	if (FReimportManager::Instance())
	{
		FReimportManager::Instance()->OnPostReimport().AddRaw(this, &FSkeletalMeshEditor::HandlePostReimport);
	}

	FPersonaToolkitArgs PersonaToolkitArgs;
	PersonaToolkitArgs.OnPreviewSceneSettingsCustomized = FOnPreviewSceneSettingsCustomized::FDelegate::CreateSP(this, &FSkeletalMeshEditor::HandleOnPreviewSceneSettingsCustomized);
	
	FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");
	PersonaToolkit = PersonaModule.CreatePersonaToolkit(InSkeletalMesh, PersonaToolkitArgs);

	PersonaToolkit->GetPreviewScene()->SetDefaultAnimationMode(EPreviewSceneDefaultAnimationMode::ReferencePose);

	PersonaModule.RecordAssetOpened(FAssetData(InSkeletalMesh));

	TSharedPtr<IPersonaPreviewScene> PreviewScene = PersonaToolkit->GetPreviewScene();

	FSkeletonTreeArgs SkeletonTreeArgs;
	SkeletonTreeArgs.OnSelectionChanged = FOnSkeletonTreeSelectionChanged::CreateSP(this, &FSkeletalMeshEditor::HandleSelectionChanged);
	SkeletonTreeArgs.PreviewScene = PreviewScene;
	SkeletonTreeArgs.ContextName = GetToolkitFName();
	SkeletonTreeArgs.bAllowInvisibleItemSelection = true;

	ISkeletonEditorModule& SkeletonEditorModule = FModuleManager::GetModuleChecked<ISkeletonEditorModule>("SkeletonEditor");
	SkeletonTree = SkeletonEditorModule.CreateSkeletonTree(PersonaToolkit->GetSkeleton(), SkeletonTreeArgs);

	MorphTargetViewer = PersonaModule.CreateDefaultMorphTargetViewerWidget(PreviewScene.ToSharedRef(), OnPostUndo);
	
	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;
	FAssetEditorToolkit::InitAssetEditor(Mode, InitToolkitHost, SkeletalMeshEditorAppIdentifier, FTabManager::FLayout::NullLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, InSkeletalMesh);
	
	if (UModeManagerInteractiveToolsContext* ToolsContext = GetEditorModeManager().GetInteractiveToolsContext())
	{
		ToolsContext->SetViewportInteractionsEnabled(true);
	}
	
	TSharedPtr<class IToolkitHost> PinnedToolkitHost = ToolkitHost.Pin();
	check(PinnedToolkitHost.IsValid());
	ModeUILayer = MakeShareable(new FSkeletalMeshEditorModeUILayer(PinnedToolkitHost.Get()));
	ModeUILayer->SetModeMenuCategory(SkeletalMeshEditorMenuCategory);
	
	BindCommands();

	AddApplicationMode(
		SkeletalMeshEditorModes::SkeletalMeshEditorMode,
		MakeShareable(new FSkeletalMeshEditorMode(SharedThis(this), SkeletonTree.ToSharedRef(), MorphTargetViewer.ToSharedRef())));

	SetCurrentMode(SkeletalMeshEditorModes::SkeletalMeshEditorMode);

	ExtendMenu();
	ExtendToolbar();
	RegenerateMenusAndToolbars();

	// Set up mesh click selection
	PreviewScene->RegisterOnMeshClick(FOnMeshClick::CreateSP(this, &FSkeletalMeshEditor::HandleMeshClick));
	PreviewScene->SetAllowMeshHitProxies(true);

	// run attached post-init delegates
	ISkeletalMeshEditorModule& SkeletalMeshEditorModule = FModuleManager::GetModuleChecked<ISkeletalMeshEditorModule>("SkeletalMeshEditor");
	const TArray<ISkeletalMeshEditorModule::FOnSkeletalMeshEditorInitialized>& PostInitDelegates = SkeletalMeshEditorModule.GetPostEditorInitDelegates();
	for (const auto& PostInitDelegate : PostInitDelegates)
	{
		PostInitDelegate.ExecuteIfBound(SharedThis<ISkeletalMeshEditor>(this));
	}
}

void FSkeletalMeshEditor::OnBuildViewportInteractions(UViewportInteractionsBehaviorSource* Source)
{
	ISkeletalMeshEditor::OnBuildViewportInteractions(Source);
	if (Source)
	{
		Source->AddInteraction<UFrustumSelectInteraction>();
	}
}

void FSkeletalMeshEditor::OnToolkitHostingStarted(const TSharedRef<IToolkit>& Toolkit)
{
	ISkeletalMeshEditor::OnToolkitHostingStarted(Toolkit);

	ModeUILayer->OnToolkitHostingStarted(Toolkit);
}

void FSkeletalMeshEditor::OnToolkitHostingFinished(const TSharedRef<IToolkit>& Toolkit)
{
	ISkeletalMeshEditor::OnToolkitHostingFinished(Toolkit);
	
	ModeUILayer->OnToolkitHostingFinished(Toolkit);
}

void FSkeletalMeshEditor::PostInitAssetEditor()
{
	ISkeletalMeshEditor::PostInitAssetEditor();

	
}

FName FSkeletalMeshEditor::GetToolkitFName() const
{
	return FName("SkeletalMeshEditor");
}

FText FSkeletalMeshEditor::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "SkeletalMeshEditor");
}

FString FSkeletalMeshEditor::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "SkeletalMeshEditor ").ToString();
}

FLinearColor FSkeletalMeshEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor(0.3f, 0.2f, 0.5f, 0.5f);
}

void FSkeletalMeshEditor::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(SkeletalMesh);
}

void FSkeletalMeshEditor::BindCommands()
{
	FSkeletalMeshEditorCommands::Register();

	constexpr bool bWithNewFileFalse = false;
	constexpr bool bWithDialogFalse = false;
	constexpr bool bWithDialogTrue = true;

	const FSkeletalMeshEditorCommands Commands = FSkeletalMeshEditorCommands::Get();
	ToolkitCommands->MapAction(Commands.ReimportMesh,
		FExecuteAction::CreateSP(this, &FSkeletalMeshEditor::HandleReimportMesh, FReimportParameters((int32)INDEX_NONE, bWithNewFileFalse, bWithDialogFalse)));

	ToolkitCommands->MapAction(Commands.ReimportAllMesh,
		FExecuteAction::CreateSP(this, &FSkeletalMeshEditor::HandleReimportAllMesh, FReimportParameters((int32)INDEX_NONE, bWithNewFileFalse, bWithDialogFalse)));

	ToolkitCommands->MapAction(Commands.ReimportWithDialog,
		FExecuteAction::CreateSP(this, &FSkeletalMeshEditor::HandleReimportMesh, FReimportParameters((int32)INDEX_NONE, bWithNewFileFalse, bWithDialogTrue)));

	ToolkitCommands->MapAction(FPersonaCommonCommands::Get().TogglePlay,
		FExecuteAction::CreateRaw(&GetPersonaToolkit()->GetPreviewScene().Get(), &IPersonaPreviewScene::TogglePlayback));

	// Bake Materials
	ToolkitCommands->MapAction(Commands.BakeMaterials,
		FExecuteAction::CreateSP(this, &FSkeletalMeshEditor::BakeMaterials),
		FCanExecuteAction());

	// Reset transforms
	static constexpr bool bSelectedOnly = true;
	ToolkitCommands->MapAction(Commands.ResetBoneTransforms,
		FExecuteAction::CreateSP(GetSkeletonTree(), &ISkeletonTree::ResetBoneTransforms, bSelectedOnly));
	
	ToolkitCommands->MapAction(Commands.ResetAllBonesTransforms,
		FExecuteAction::CreateSP(GetSkeletonTree(), &ISkeletonTree::ResetBoneTransforms, !bSelectedOnly));
}

TSharedPtr<FSkeletalMeshEditor> FSkeletalMeshEditor::GetSkeletalMeshEditor(const FToolMenuContext& InMenuContext)
{
	if (USkeletalMeshToolMenuContext* Context = InMenuContext.FindContext<USkeletalMeshToolMenuContext>())
	{
		if (Context->SkeletalMeshEditor.IsValid())
		{
			return StaticCastSharedPtr<FSkeletalMeshEditor>(Context->SkeletalMeshEditor.Pin());
		}
	}

	return TSharedPtr<FSkeletalMeshEditor>();
}

void FSkeletalMeshEditor::RegisterReimportContextMenu(const FName InBaseMenuName, bool bWithDialog)
{
	static auto ReimportMeshWithNewFileAction = [](const FToolMenuContext& InMenuContext, int32 SourceFileIndex, bool bWithDialog)
	{
		constexpr bool bWithNewFileTrue = true;
		TSharedPtr<FSkeletalMeshEditor> SkeletalMeshEditor = GetSkeletalMeshEditor(InMenuContext);
		if (SkeletalMeshEditor.IsValid())
		{
			if (USkeletalMesh* FoundSkeletalMesh = SkeletalMeshEditor->SkeletalMesh)
			{
				if (UFbxSkeletalMeshImportData* SkeletalMeshImportData = Cast<UFbxSkeletalMeshImportData>(FoundSkeletalMesh->GetAssetImportData()))
				{
					SkeletalMeshImportData->ImportContentType = SourceFileIndex == 0 ? EFBXImportContentType::FBXICT_All : SourceFileIndex == 1 ? EFBXImportContentType::FBXICT_Geometry : EFBXImportContentType::FBXICT_SkinningWeights;
				}
				
				SkeletalMeshEditor->HandleReimportMesh(FReimportParameters(SourceFileIndex, bWithNewFileTrue, bWithDialog));
			}
		}
	};

	static auto ReimportAction = [](const FToolMenuContext& InMenuContext, const int32 SourceFileIndex, const bool bReimportAll, const bool bWithNewFile, const bool bWithDialog)
	{
		TSharedPtr<FSkeletalMeshEditor> SkeletalMeshEditor = GetSkeletalMeshEditor(InMenuContext);
		if (SkeletalMeshEditor.IsValid())
		{
			if (USkeletalMesh* FoundSkeletalMesh = SkeletalMeshEditor->SkeletalMesh)
			{
				UFbxSkeletalMeshImportData* SkeletalMeshImportData = FoundSkeletalMesh ? Cast<UFbxSkeletalMeshImportData>(FoundSkeletalMesh->GetAssetImportData()) : nullptr;
				if (SkeletalMeshImportData)
				{
					SkeletalMeshImportData->ImportContentType = SourceFileIndex == 0 ? EFBXImportContentType::FBXICT_All : SourceFileIndex == 1 ? EFBXImportContentType::FBXICT_Geometry : EFBXImportContentType::FBXICT_SkinningWeights;
				}

				if (bReimportAll)
				{
					SkeletalMeshEditor->HandleReimportAllMesh(FReimportParameters(SourceFileIndex, bWithNewFile, bWithDialog));
				}
				else
				{
					SkeletalMeshEditor->HandleReimportMesh(FReimportParameters(SourceFileIndex, bWithNewFile, bWithDialog));
				}
			}
		}
	};

	static auto CreateReimportSubMenu = [](UToolMenu* InMenu, bool bReimportAll, bool bWithNewFile, bool bWithDialog)
	{
		TSharedPtr<FSkeletalMeshEditor> SkeletalMeshEditor = GetSkeletalMeshEditor(InMenu->Context);
		if (SkeletalMeshEditor.IsValid())
		{
			USkeletalMesh* InSkeletalMesh = SkeletalMeshEditor->SkeletalMesh;
			if (InSkeletalMesh && InSkeletalMesh->GetAssetImportData())
			{
				//Get the data
				TArray<FString> SourceFilePaths;
				InSkeletalMesh->GetAssetImportData()->ExtractFilenames(SourceFilePaths);
				TArray<FString> SourceFileLabels;
				InSkeletalMesh->GetAssetImportData()->ExtractDisplayLabels(SourceFileLabels);

				if (SourceFileLabels.Num() > 0 && SourceFileLabels.Num() == SourceFilePaths.Num())
				{
					FToolMenuSection& Section = InMenu->AddSection("Reimport");
					for (int32 SourceFileIndex = 0; SourceFileIndex < SourceFileLabels.Num(); ++SourceFileIndex)
					{
						FText ReimportLabel = FText::Format(LOCTEXT("ReimportNoLabel", "SourceFile {0}"), SourceFileIndex);
						FText ReimportLabelTooltip = FText::Format(LOCTEXT("ReimportNoLabelTooltip", "Reimport File: {0}"), FText::FromString(SourceFilePaths[SourceFileIndex]));
						if (SourceFileLabels[SourceFileIndex].Len() > 0)
						{
							ReimportLabel = FText::Format(LOCTEXT("ReimportLabel", "{0}"), FText::FromString(SourceFileLabels[SourceFileIndex]));
							ReimportLabelTooltip = FText::Format(LOCTEXT("ReimportLabelTooltip", "Reimport {0} File: {1}"), FText::FromString(SourceFileLabels[SourceFileIndex]), FText::FromString(SourceFilePaths[SourceFileIndex]));
						}

						Section.AddMenuEntry(
							NAME_None,
							ReimportLabel,
							ReimportLabelTooltip,
							FSlateIcon(FAppStyle::GetAppStyleSetName(), "ContentBrowser.AssetActions.ReimportAsset"),
							FToolMenuExecuteAction::CreateLambda(ReimportAction, SourceFileIndex, bReimportAll, bWithNewFile, bWithDialog)
						);
					}
				}
			}
		}
	};

	if (!UToolMenus::Get()->IsMenuRegistered(UToolMenus::JoinMenuPaths(InBaseMenuName, bWithDialog ? "ReimportWithDialogContextMenu" : "ReimportContextMenu")))
	{
		UToolMenu* ToolMenu = UToolMenus::Get()->RegisterMenu(UToolMenus::JoinMenuPaths(InBaseMenuName, bWithDialog ? "ReimportWithDialogContextMenu" : "ReimportContextMenu"));
		ToolMenu->AddDynamicSection("Section", FNewToolMenuDelegate::CreateLambda([bWithDialog](UToolMenu* InMenu)
		{
			TSharedPtr<FSkeletalMeshEditor> SkeletalMeshEditor = GetSkeletalMeshEditor(InMenu->Context);
			if (SkeletalMeshEditor.IsValid())
			{
				USkeletalMesh* InSkeletalMesh = SkeletalMeshEditor->SkeletalMesh;
				bool bShowSubMenu = InSkeletalMesh != nullptr && InSkeletalMesh->GetAssetImportData() != nullptr && InSkeletalMesh->GetAssetImportData()->GetSourceFileCount() > 1;

				constexpr bool bWithNewFileFalse = false;
				constexpr bool bWithNewFileTrue = true;

				FToolMenuSection& Section = InMenu->AddSection("Section");
				if (!bShowSubMenu)
				{
					//Reimport
					Section.AddMenuEntry(
						FSkeletalMeshEditorCommands::Get().ReimportMesh->GetCommandName(),
						FSkeletalMeshEditorCommands::Get().ReimportMesh->GetLabel(),
						FSkeletalMeshEditorCommands::Get().ReimportMesh->GetDescription(),
						FSlateIcon(),
						FUIAction(FExecuteAction::CreateSP(SkeletalMeshEditor.ToSharedRef(), &FSkeletalMeshEditor::HandleReimportMesh, FReimportParameters(0, bWithNewFileFalse, bWithDialog))));

					Section.AddMenuEntry(
						FSkeletalMeshEditorCommands::Get().ReimportMeshWithNewFile->GetCommandName(),
						FSkeletalMeshEditorCommands::Get().ReimportMeshWithNewFile->GetLabel(),
						FSkeletalMeshEditorCommands::Get().ReimportMeshWithNewFile->GetDescription(),
						FSlateIcon(),
						FUIAction(FExecuteAction::CreateSP(SkeletalMeshEditor.ToSharedRef(), &FSkeletalMeshEditor::HandleReimportMesh, FReimportParameters(0, bWithNewFileTrue, bWithDialog))));

					//Reimport ALL
					Section.AddMenuEntry(
						FSkeletalMeshEditorCommands::Get().ReimportAllMesh->GetCommandName(),
						FSkeletalMeshEditorCommands::Get().ReimportAllMesh->GetLabel(),
						FSkeletalMeshEditorCommands::Get().ReimportAllMesh->GetDescription(),
						FSlateIcon(),
						FUIAction(FExecuteAction::CreateSP(SkeletalMeshEditor.ToSharedRef(), &FSkeletalMeshEditor::HandleReimportAllMesh, FReimportParameters(0, bWithNewFileFalse, bWithDialog))));

					Section.AddMenuEntry(
						FSkeletalMeshEditorCommands::Get().ReimportAllMeshWithNewFile->GetCommandName(),
						FSkeletalMeshEditorCommands::Get().ReimportAllMeshWithNewFile->GetLabel(),
						FSkeletalMeshEditorCommands::Get().ReimportAllMeshWithNewFile->GetDescription(),
						FSlateIcon(),
						FUIAction(FExecuteAction::CreateSP(SkeletalMeshEditor.ToSharedRef(), &FSkeletalMeshEditor::HandleReimportAllMesh, FReimportParameters(0, bWithNewFileTrue, bWithDialog))));

					Section.AddSubMenu(
						"ReimportMultiSources",
						LOCTEXT("ReimportMultiSources", "Reimport Content"),
						LOCTEXT("ReimportMultiSourcesTooltip", "Reimport Geometry or Skinning Weights content, this will create multi import source file."),
						FNewToolMenuDelegate::CreateLambda([bWithDialog](UToolMenu* InMenu)
							{
								FToolMenuSection& Section = InMenu->AddSection("Reimport");

								TSharedPtr<FSkeletalMeshEditor> SkeletalMeshEditor = GetSkeletalMeshEditor(InMenu->Context);
								if (SkeletalMeshEditor.IsValid())
								{
									Section.AddMenuEntry(
										"ReimportGeometryContentLabel",
										LOCTEXT("ReimportGeometryContentLabel", "Geometry"),
										LOCTEXT("ReimportGeometryContentLabelTooltipTooltip", "Reimport Geometry Only"),
										FSlateIcon(FAppStyle::GetAppStyleSetName(), "ContentBrowser.AssetActions.ReimportAsset"),
										FToolMenuExecuteAction::CreateLambda(ReimportMeshWithNewFileAction, 1, bWithDialog)
									);

									Section.AddMenuEntry(
										"ReimportSkinningAndWeightsContentLabel",
										LOCTEXT("ReimportSkinningAndWeightsContentLabel", "Skinning And Weights"),
										LOCTEXT("ReimportSkinningAndWeightsContentLabelTooltipTooltip", "Reimport Skinning And Weights Only"),
										FSlateIcon(FAppStyle::GetAppStyleSetName(), "ContentBrowser.AssetActions.ReimportAsset"),
										FToolMenuExecuteAction::CreateLambda(ReimportMeshWithNewFileAction, 2, bWithDialog)
									);
								}
							}));
				}
				else
				{
					//Create 4 submenu: Reimport, ReimportWithNewFile, ReimportAll and ReimportAllWithNewFile
					Section.AddSubMenu(
						FSkeletalMeshEditorCommands::Get().ReimportMesh->GetCommandName(),
						FSkeletalMeshEditorCommands::Get().ReimportMesh->GetLabel(),
						FSkeletalMeshEditorCommands::Get().ReimportMesh->GetDescription(),
						FNewToolMenuDelegate::CreateLambda(CreateReimportSubMenu, false, false, bWithDialog));
			
					Section.AddSubMenu(
						FSkeletalMeshEditorCommands::Get().ReimportMeshWithNewFile->GetCommandName(),
						FSkeletalMeshEditorCommands::Get().ReimportMeshWithNewFile->GetLabel(),
						FSkeletalMeshEditorCommands::Get().ReimportMeshWithNewFile->GetDescription(),
						FNewToolMenuDelegate::CreateLambda(CreateReimportSubMenu, false, true, bWithDialog));

					Section.AddSubMenu(
						FSkeletalMeshEditorCommands::Get().ReimportAllMesh->GetCommandName(),
						FSkeletalMeshEditorCommands::Get().ReimportAllMesh->GetLabel(),
						FSkeletalMeshEditorCommands::Get().ReimportAllMesh->GetDescription(),
						FNewToolMenuDelegate::CreateLambda(CreateReimportSubMenu, true, false, bWithDialog));

					Section.AddSubMenu(
						FSkeletalMeshEditorCommands::Get().ReimportAllMeshWithNewFile->GetCommandName(),
						FSkeletalMeshEditorCommands::Get().ReimportAllMeshWithNewFile->GetLabel(),
						FSkeletalMeshEditorCommands::Get().ReimportAllMeshWithNewFile->GetDescription(),
						FNewToolMenuDelegate::CreateLambda(CreateReimportSubMenu, true, true, bWithDialog));
				}
			}
		}));
	}
}

void FSkeletalMeshEditor::ExtendToolbar()
{

	// Add in Editor Specific functionality
	FName ParentName;
	static const FName MenuName = GetToolMenuToolbarName(ParentName);

	constexpr bool bWithDialogFalse = false;
	constexpr bool bWithDialogTrue = true;
	RegisterReimportContextMenu(MenuName, bWithDialogFalse);
	RegisterReimportContextMenu(MenuName, bWithDialogTrue);

	UToolMenu* ToolMenu = UToolMenus::Get()->ExtendMenu(MenuName);
	const FToolMenuInsert SectionInsertLocation("Asset", EToolMenuInsertType::After);

	{
		ToolMenu->AddDynamicSection("Persona", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InToolMenu)
		{
			TSharedPtr<FSkeletalMeshEditor> SkeletalMeshEditor = GetSkeletalMeshEditor(InToolMenu->Context);
			if (SkeletalMeshEditor.IsValid() && SkeletalMeshEditor->PersonaToolkit.IsValid())
			{
				FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");
				FPersonaModule::FCommonToolbarExtensionArgs Args;
				Args.bPreviewMesh = false;
				PersonaModule.AddCommonToolbarExtensions(InToolMenu, Args);
			}
		}), SectionInsertLocation);
	}

	//Reimport button
	{
		FToolMenuSection& Section = ToolMenu->AddSection("Mesh", FText(), SectionInsertLocation);
		
		Section.AddEntry(FToolMenuEntry::InitToolBarButton(FSkeletalMeshEditorCommands::Get().ReimportMesh));

		Section.AddEntry(FToolMenuEntry::InitComboButton("ReimportContextMenu", FUIAction(), FNewToolMenuDelegate(), FText(), FText(), FSlateIcon(), true));
		
		Section.AddEntry(FToolMenuEntry::InitToolBarButton(FSkeletalMeshEditorCommands::Get().ReimportWithDialog));

		Section.AddEntry(FToolMenuEntry::InitComboButton("ReimportWithDialogContextMenu", FUIAction(), FNewToolMenuDelegate(), FText(), FText(), FSlateIcon(), true));
	}

	// If the ToolbarExtender is valid, remove it before rebuilding it
	if (ToolbarExtender.IsValid())
	{
		RemoveToolbarExtender(ToolbarExtender);
		ToolbarExtender.Reset();
	}

	ToolbarExtender = MakeShareable(new FExtender);

	AddToolbarExtender(ToolbarExtender);

	ISkeletalMeshEditorModule& SkeletalMeshEditorModule = FModuleManager::GetModuleChecked<ISkeletalMeshEditorModule>("SkeletalMeshEditor");
	AddToolbarExtender(SkeletalMeshEditorModule.GetToolBarExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));

	TArray<ISkeletalMeshEditorModule::FSkeletalMeshEditorToolbarExtender> ToolbarExtenderDelegates = SkeletalMeshEditorModule.GetAllSkeletalMeshEditorToolbarExtenders();

	for (auto& ToolbarExtenderDelegate : ToolbarExtenderDelegates)
	{
		if (ToolbarExtenderDelegate.IsBound())
		{
			AddToolbarExtender(ToolbarExtenderDelegate.Execute(GetToolkitCommands(), SharedThis(this)));
		}
	}

	ToolbarExtender->AddToolBarExtension(
		"Asset",
		EExtensionHook::After,
		GetToolkitCommands(),
		FToolBarExtensionDelegate::CreateLambda([this](FToolBarBuilder& ParentToolbarBuilder)
	{
		// Second toolbar on right side
		FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");
		TSharedRef<class IAssetFamily> AssetFamily = PersonaModule.CreatePersonaAssetFamily(SkeletalMesh);
		AddToolbarWidget(PersonaModule.CreateAssetFamilyShortcutWidget(SharedThis(this), AssetFamily));
	}
	));
}

void FSkeletalMeshEditor::InitToolMenuContext(FToolMenuContext& MenuContext)
{
	FAssetEditorToolkit::InitToolMenuContext(MenuContext);

	USkeletalMeshToolMenuContext* Context = NewObject<USkeletalMeshToolMenuContext>();
	Context->SkeletalMeshEditor = SharedThis(this);
	MenuContext.AddObject(Context);
	
	UPersonaToolMenuContext* PersonaContext = NewObject<UPersonaToolMenuContext>();
	PersonaContext->SetToolkit(GetPersonaToolkit());
	MenuContext.AddObject(PersonaContext);
}

void FSkeletalMeshEditor::AddViewportOverlayWidget(TSharedRef<SWidget> InOverlaidWidget, int32 ZOrder)
{
	if (Viewport.IsValid())
	{
		Viewport->AddOverlayWidget(InOverlaidWidget, ZOrder);
	}	
}


void FSkeletalMeshEditor::RemoveViewportOverlayWidget(TSharedRef<SWidget> InOverlaidWidget)
{
	if (Viewport.IsValid())
	{
		Viewport->RemoveOverlayWidget(InOverlaidWidget);
	}	
}

bool FSkeletalMeshEditor::ProcessCommandBindings(const FKeyEvent& InKeyEvent) const
{
	if (HostedToolkit.IsValid() && HostedToolkit->ProcessCommandBindings(InKeyEvent))
	{
		return true;
	}

	return ISkeletalMeshEditor::ProcessCommandBindings(InKeyEvent);
}


void FSkeletalMeshEditor::OnRemoveSectionFromLodAndBelowMenuItemClicked(int32 LodIndex, int32 SectionIndex)
{
	if (SkeletalMesh == nullptr || !SkeletalMesh->GetImportedModel()->LODModels.IsValidIndex(LodIndex) || !SkeletalMesh->GetImportedModel()->LODModels[LodIndex].Sections.IsValidIndex(SectionIndex))
	{
		return;
	}
	const FSkeletalMeshLODInfo* SkeletalMeshLODInfo = SkeletalMesh->GetLODInfo(LodIndex);
	if (SkeletalMeshLODInfo == nullptr)
	{
		return;
	}
	FScopedTransaction Transaction(LOCTEXT("ChangeGenerateUpTo", "Set Generate Up To"));
	SkeletalMesh->Modify();

	SkeletalMesh->GetImportedModel()->LODModels[LodIndex].Sections[SectionIndex].GenerateUpToLodIndex = LodIndex;
	FSkeletalMeshUpdateContext UpdateContext;
	UpdateContext.SkeletalMesh = SkeletalMesh;
	UpdateContext.AssociatedComponents.Push(GetPersonaToolkit()->GetPreviewMeshComponent());
	//Generate only the LODs that can be affected by the changes
	TArray<int32> BaseLodIndexes;
	BaseLodIndexes.Add(LodIndex);
	for (int32 GenerateLodIndex = LodIndex + 1; GenerateLodIndex < SkeletalMesh->GetImportedModel()->LODModels.Num(); ++GenerateLodIndex)
	{
		const FSkeletalMeshLODInfo* CurrentSkeletalMeshLODInfo = SkeletalMesh->GetLODInfo(GenerateLodIndex);
		if (CurrentSkeletalMeshLODInfo != nullptr && CurrentSkeletalMeshLODInfo->bHasBeenSimplified && BaseLodIndexes.Contains(CurrentSkeletalMeshLODInfo->ReductionSettings.BaseLOD))
		{
			FLODUtilities::SimplifySkeletalMeshLOD(UpdateContext, GenerateLodIndex, GetTargetPlatformManagerRef().GetRunningTargetPlatform(), true);
			BaseLodIndexes.Add(GenerateLodIndex);
		}
	}
	SkeletalMesh->PostEditChange();
	GetPersonaToolkit()->GetPreviewScene()->InvalidateViews();
}

void FSkeletalMeshEditor::FillApplyClothingAssetMenu(FMenuBuilder& MenuBuilder, int32 InLodIndex, int32 InSectionIndex)
{
	if (const USkeletalMesh* const Mesh = GetPersonaToolkit()->GetPreviewMesh())
	{
		MenuBuilder.BeginSection(TEXT("ApplyClothingMenu"), LOCTEXT("ApplyClothingMenuHeader", "Available Assets"));
		{
			for (UClothingAssetBase* const Asset : Mesh->GetMeshClothingAssets())
			{
				if (Asset)  // Asset could be null if the relevant plugin isn't loaded
				{
					FUIAction Action;
					Action.CanExecuteAction = FCanExecuteAction::CreateSP(this, &FSkeletalMeshEditor::CanApplyClothing, InLodIndex, InSectionIndex);

					if (const UClothingAssetCommon* const AssetCommon = Cast<UClothingAssetCommon>(Asset))
					{
						const int32 NumClothLods = AssetCommon->GetNumLods();
						for (int32 ClothLodIndex = 0; ClothLodIndex < NumClothLods; ++ClothLodIndex)
						{
							Action.ExecuteAction = FExecuteAction::CreateSP(this, &FSkeletalMeshEditor::OnApplyClothingAssetClicked, Asset, InLodIndex, InSectionIndex, ClothLodIndex);

							MenuBuilder.AddMenuEntry(
								FText::Format(LOCTEXT("ApplyClothingMenuItem", "{0} - LOD{1}"), FText::FromString(Asset->GetName()), FText::AsNumber(ClothLodIndex)),
								LOCTEXT("ApplyClothingMenuItem_ToolTip", "Apply this clothing data to the selected mesh LOD and section"),
								FSlateIcon(),
								Action);
						}
					}
					else
					{
						const int32 ClothLodIndex = InLodIndex;  // Use the same LOD as the cloth for all other types of clothing data
						Action.ExecuteAction = FExecuteAction::CreateSP(this, &FSkeletalMeshEditor::OnApplyClothingAssetClicked, Asset, InLodIndex, InSectionIndex, ClothLodIndex);

						MenuBuilder.AddMenuEntry(
							FText::FromString(Asset->GetName()),
							LOCTEXT("ApplyClothingMenuItem_ToolTip", "Apply this clothing data to the selected mesh LOD and section"),
							FSlateIcon(),
							Action);
					}
				}
			}
		}
		MenuBuilder.EndSection();
	}
}

void FSkeletalMeshEditor::FillCreateClothingMenu(FMenuBuilder& MenuBuilder, int32 InLodIndex, int32 InSectionIndex)
{
	USkeletalMesh* Mesh = GetPersonaToolkit()->GetPreviewMesh();

	if(!Mesh)
	{
		return;
	}

	TSharedRef<SWidget> Widget = SNew(SCreateClothingSettingsPanel)
		.Mesh(Mesh)
		.MeshName(Mesh->GetName())
		.LodIndex(InLodIndex)
		.SectionIndex(InSectionIndex)
		.OnCreateRequested(this, &FSkeletalMeshEditor::OnCreateClothingAssetMenuItemClicked)
		.bIsSubImport(false);

	MenuBuilder.AddWidget(Widget, FText::GetEmpty(), true, false);
}

void FSkeletalMeshEditor::FillCreateClothingLodMenu(FMenuBuilder& MenuBuilder, int32 InLodIndex, int32 InSectionIndex)
{
	USkeletalMesh* Mesh = GetPersonaToolkit()->GetPreviewMesh();

	if(!Mesh)
	{
		return;
	}

	TSharedRef<SWidget> Widget = SNew(SCreateClothingSettingsPanel)
		.Mesh(Mesh)
		.MeshName(Mesh->GetName())
		.LodIndex(InLodIndex)
		.SectionIndex(InSectionIndex)
		.OnCreateRequested(this, &FSkeletalMeshEditor::OnCreateClothingAssetMenuItemClicked)
		.bIsSubImport(true);

		MenuBuilder.AddWidget(Widget, FText::GetEmpty(), true, false);
}

void FSkeletalMeshEditor::OnRemoveClothingAssetMenuItemClicked(int32 InLodIndex, int32 InSectionIndex)
{
	RemoveClothing(InLodIndex, InSectionIndex);
}

void FSkeletalMeshEditor::OnCreateClothingAssetMenuItemClicked(FSkeletalMeshClothBuildParams& Params)
{
	// Close the menu we created
	FSlateApplication::Get().DismissAllMenus();

	USkeletalMesh* Mesh = GetPersonaToolkit()->GetPreviewMesh();

	if(Mesh)
	{
		Mesh->Modify();

		FScopedSuspendAlternateSkinWeightPreview ScopedSuspendAlternateSkinnWeightPreview(Mesh);

		if (Params.bRemoveFromMesh)  // Remove section prior to importing, otherwise the UsedBoneIndices won't be reflecting the loss of the section in the sub LOD
		{
			// Force the rebuilding of the render data at the end of this scope to update the used bone array
			FScopedSkeletalMeshPostEditChange ScopedSkeletalMeshPostEditChange(Mesh);
	
			// User doesn't want the section anymore as a renderable, get rid of it
			Mesh->RemoveMeshSection(Params.LodIndex, Params.SourceSection);
		}

		// Update the skeletal mesh at the end of the scope, this time with the new clothing changes
		FScopedSkeletalMeshPostEditChange ScopedSkeletalMeshPostEditChange(Mesh);

		// Handle the creation through the clothing asset factory
		FClothingSystemEditorInterfaceModule& ClothingEditorModule = FModuleManager::LoadModuleChecked<FClothingSystemEditorInterfaceModule>("ClothingSystemEditorInterface");
		UClothingAssetFactoryBase* AssetFactory = ClothingEditorModule.GetClothingAssetFactory();

		// See if we're importing a LOD or new asset
		if(Params.TargetAsset.IsValid())
		{
			UClothingAssetBase* TargetAssetPtr = Params.TargetAsset.Get();
			int32 SectionIndex = -1, AssetLodIndex = -1;
			if (Params.bRemapParameters)
			{
				if (TargetAssetPtr && Mesh->GetImportedModel()->LODModels.IsValidIndex(Params.TargetLod))
				{
					//Cache the section and asset LOD this asset was bound at before unbinding
					FSkeletalMeshLODModel& SkelLod = Mesh->GetImportedModel()->LODModels[Params.TargetLod];
					for (int32 i = 0; i < SkelLod.Sections.Num(); ++i)
					{
						if (SkelLod.Sections[i].ClothingData.AssetGuid == TargetAssetPtr->GetAssetGuid())
						{
							SectionIndex = i;
							AssetLodIndex = SkelLod.Sections[i].ClothingData.AssetLodIndex;
							RemoveClothing(Params.TargetLod, SectionIndex);
							break;
						}
					}
				}
			}

			AssetFactory->ImportLodToClothing(Mesh, Params);

			if (Params.bRemapParameters)
			{
				//If it was bound previously, rebind at same section with same LOD
				if (TargetAssetPtr && SectionIndex > -1)
				{
					ApplyClothing(TargetAssetPtr, Params.TargetLod, SectionIndex, AssetLodIndex);
				}
			}
		}
		else
		{
			UClothingAssetBase* NewClothingAsset = AssetFactory->CreateFromSkeletalMesh(Mesh, Params);

			if(NewClothingAsset)
			{
				Mesh->AddClothingAsset(NewClothingAsset);
			}
		}

		//Make sure no section is isolated or highlighted
		UDebugSkelMeshComponent * MeshComponent = GetPersonaToolkit()->GetPreviewScene()->GetPreviewMeshComponent();
		if(MeshComponent)
		{
			MeshComponent->SetSelectedEditorSection(INDEX_NONE);
			MeshComponent->SetSelectedEditorMaterial(INDEX_NONE);
			MeshComponent->SetMaterialPreview(INDEX_NONE);
			MeshComponent->SetSectionPreview(INDEX_NONE);
		}
	}
}

void FSkeletalMeshEditor::OnApplyClothingAssetClicked(UClothingAssetBase* InAssetToApply, int32 InMeshLodIndex, int32 InMeshSectionIndex, int32 InClothLodIndex)
{
	ApplyClothing(InAssetToApply, InMeshLodIndex, InMeshSectionIndex, InClothLodIndex);
}

bool FSkeletalMeshEditor::CanApplyClothing(int32 InLodIndex, int32 InSectionIndex)
{
	USkeletalMesh* Mesh = GetPersonaToolkit()->GetPreviewMesh();

	if(Mesh->GetMeshClothingAssets().Num() > 0)
	{
	FSkeletalMeshModel* MeshResource = Mesh->GetImportedModel();

	if(MeshResource->LODModels.IsValidIndex(InLodIndex))
	{
		FSkeletalMeshLODModel& LodModel = MeshResource->LODModels[InLodIndex];

		if(LodModel.Sections.IsValidIndex(InSectionIndex))
		{
			return !LodModel.Sections[InSectionIndex].HasClothingData();
		}
	}
	}

	return false;
}

bool FSkeletalMeshEditor::CanRemoveClothing(int32 InLodIndex, int32 InSectionIndex)
{
	USkeletalMesh* Mesh = GetPersonaToolkit()->GetPreviewMesh();

	FSkeletalMeshModel* MeshResource = Mesh->GetImportedModel();

	if(MeshResource->LODModels.IsValidIndex(InLodIndex))
	{
		FSkeletalMeshLODModel& LodModel = MeshResource->LODModels[InLodIndex];

		if(LodModel.Sections.IsValidIndex(InSectionIndex))
		{
			return LodModel.Sections[InSectionIndex].HasClothingData();
		}
	}

	return false;
}

bool FSkeletalMeshEditor::CanCreateClothing(int32 InLodIndex, int32 InSectionIndex)
{
	USkeletalMesh* Mesh = GetPersonaToolkit()->GetPreviewMesh();

	FSkeletalMeshModel* MeshResource = Mesh->GetImportedModel();

	if(MeshResource->LODModels.IsValidIndex(InLodIndex))
	{
		FSkeletalMeshLODModel& LodModel = MeshResource->LODModels[InLodIndex];

		if(LodModel.Sections.IsValidIndex(InSectionIndex))
		{
			FSkelMeshSection& Section = LodModel.Sections[InSectionIndex];

			return !Section.HasClothingData();
		}
	}

	return false;
}

bool FSkeletalMeshEditor::CanCreateClothingLod(int32 InLodIndex, int32 InSectionIndex)
{
	USkeletalMesh* Mesh = GetPersonaToolkit()->GetPreviewMesh();

	return Mesh && Mesh->GetMeshClothingAssets().Num() > 0 && CanApplyClothing(InLodIndex, InSectionIndex);
}

void FSkeletalMeshEditor::ApplyClothing(UClothingAssetBase* InAsset, int32 InLodIndex, int32 InSectionIndex, int32 InClothingLod)
{
	USkeletalMesh* Mesh = GetPersonaToolkit()->GetPreviewMesh();

	if (Mesh == nullptr)
	{
		return;
	}
	FScopedSuspendAlternateSkinWeightPreview ScopedSuspendAlternateSkinnWeightPreview(Mesh);
	{
		if (InAsset)
		{
			FSkeletalMeshClothingSystemUtilities::AssignClothingToSection(Mesh, InAsset, InLodIndex, InSectionIndex, InClothingLod, nullptr);
		}
		else
		{
			FSkeletalMeshClothingSystemUtilities::RemoveClothingFromSection(Mesh, InLodIndex, InSectionIndex, nullptr);
		}
	}
}

void FSkeletalMeshEditor::RemoveClothing(int32 InLodIndex, int32 InSectionIndex)
{
	USkeletalMesh* const Mesh = GetPersonaToolkit()->GetPreviewMesh();

	if (!Mesh)
	{
		return;
	}
	FScopedSuspendAlternateSkinWeightPreview ScopedSuspendAlternateSkinnWeightPreview(Mesh);
	{
		FSkeletalMeshClothingSystemUtilities::RemoveClothingFromSection(Mesh, InLodIndex, InSectionIndex, nullptr);
	}
}

void FSkeletalMeshEditor::ExtendMenu()
{
	MenuExtender = MakeShareable(new FExtender);

	AddMenuExtender(MenuExtender);

	ISkeletalMeshEditorModule& SkeletalMeshEditorModule = FModuleManager::GetModuleChecked<ISkeletalMeshEditorModule>("SkeletalMeshEditor");
	AddMenuExtender(SkeletalMeshEditorModule.GetMenuExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));

	FToolMenuOwnerScoped OwnerScoped(this);

	UToolMenu* AssetMenu = UToolMenus::Get()->ExtendMenu("MainFrame.MainMenu.Asset");
	{
		FToolMenuSection& AssetSection = AssetMenu->FindOrAddSection("AssetEditorActions");
		const FName SkeletalMeshToolkitName = GetToolkitFName();
		AssetSection.AddDynamicEntry("AssetManagerEditorSkeletalMeshCommands", FNewToolMenuSectionDelegate::CreateLambda([SkeletalMeshToolkitName](FToolMenuSection& InSection)
			{
				UAssetEditorToolkitMenuContext* MenuContext = InSection.FindContext<UAssetEditorToolkitMenuContext>();
				if (MenuContext && MenuContext->Toolkit.IsValid() && MenuContext->Toolkit.Pin()->GetToolkitFName() == SkeletalMeshToolkitName)
				{
					InSection.AddMenuEntry(FSkeletalMeshEditorCommands::Get().BakeMaterials);
				}
			}
		));
	}


	if (!UToolMenus::Get()->IsMenuRegistered(ViewportMenuId))
	{
		UToolMenu* ViewportMenu = UToolMenus::Get()->RegisterMenu(ViewportMenuId);
		FToolMenuSection& AssetSection = ViewportMenu->AddSection("Asset", LOCTEXT("MeshClickMenu_Section_Asset", "Asset"));
		AssetSection.AddDynamicEntry("DynamicAssetEntries", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
		{
			if (USkeletalMeshEditorContextMenuContext* MenuContext = InSection.FindContext<USkeletalMeshEditorContextMenuContext>())
			{
				const int32 LodIndex = MenuContext->LodIndex;
				const int32 SectionIndex = MenuContext->SectionIndex;

				TSharedRef<SWidget> InfoWidget = SNew(SBox)
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Fill)
					.Padding(FMargin(2.5f, 5.0f, 2.5f, 0.0f))
					[
						SNew(SBorder)
						.HAlign(HAlign_Fill)
						.VAlign(VAlign_Fill)
						//.Padding(FMargin(0.0f, 10.0f, 0.0f, 0.0f))
						.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
						[
							SNew(SBox)
							.HAlign(HAlign_Center)
							.VAlign(VAlign_Center)
							[
								SNew(STextBlock)
								.Font(FAppStyle::GetFontStyle("CurveEd.LabelFont"))
								.Text(FText::Format(LOCTEXT("MeshClickMenu_SectionInfo", "LOD{0} - Section {1}"), LodIndex, SectionIndex))
							]
						]
					];

				InSection.AddEntry(FToolMenuEntry::InitWidget("InfoWidget", InfoWidget, FText::GetEmpty(), true, false, true));
			}
		}));

		FToolMenuSection& ClothSection = ViewportMenu->AddSection("Clothing", LOCTEXT("MeshClickMenu_Section_Clothing", "Clothing"));
		ClothSection.AddDynamicEntry("DynamicClothingEntries", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
		{
			if (USkeletalMeshEditorContextMenuContext* MenuContext = InSection.FindContext<USkeletalMeshEditorContextMenuContext>())
			{
				TSharedPtr<FSkeletalMeshEditor> Editor = GetSkeletalMeshEditor(InSection.Context);
				if(Editor.IsValid())
				{
					const int32 LodIndex = MenuContext->LodIndex;
					const int32 SectionIndex = MenuContext->SectionIndex;

					InSection.AddSubMenu(
						"ApplyClothing",
						LOCTEXT("MeshClickMenu_AssetApplyMenu", "Apply Clothing Data..."),
						LOCTEXT("MeshClickMenu_AssetApplyMenu_ToolTip", "Select clothing data to apply to the selected section."),
						FNewToolMenuChoice(FNewMenuDelegate::CreateSP(Editor.Get(), &FSkeletalMeshEditor::FillApplyClothingAssetMenu, LodIndex, SectionIndex)),
						FToolUIActionChoice(FUIAction(
							FExecuteAction(),
							FCanExecuteAction::CreateSP(Editor.Get(), &FSkeletalMeshEditor::CanApplyClothing, LodIndex, SectionIndex)
						)),
						EUserInterfaceActionType::Button
					);

					InSection.AddMenuEntry(
						"RemoveClothing",
						LOCTEXT("MeshClickMenu_RemoveClothing", "Remove Clothing Data"),
						LOCTEXT("MeshClickMenu_RemoveClothing_ToolTip", "Remove the currently assigned clothing data."),
						FSlateIcon(),
						FToolUIActionChoice(FUIAction(
							FExecuteAction::CreateSP(Editor.Get(), &FSkeletalMeshEditor::OnRemoveClothingAssetMenuItemClicked, LodIndex, SectionIndex),
							FCanExecuteAction::CreateSP(Editor.Get(), &FSkeletalMeshEditor::CanRemoveClothing, LodIndex, SectionIndex)
						))
					);

					InSection.AddSubMenu(
						"CreateClothing",
						LOCTEXT("MeshClickMenu_CreateClothing_Label", "Create Clothing Data from Section"),
						LOCTEXT("MeshClickMenu_CreateClothing_ToolTip", "Create a new clothing data using the selected section as a simulation mesh"),
						FNewToolMenuChoice(FNewMenuDelegate::CreateSP(Editor.Get(), &FSkeletalMeshEditor::FillCreateClothingMenu, LodIndex, SectionIndex)),
						FToolUIActionChoice(FUIAction(
							FExecuteAction(),
							FCanExecuteAction::CreateSP(Editor.Get(), &FSkeletalMeshEditor::CanCreateClothing, LodIndex, SectionIndex)
						)),
						EUserInterfaceActionType::Button
					);

					InSection.AddSubMenu(
						"CreateClothingLOD",
						LOCTEXT("MeshClickMenu_CreateClothingNewLod_Label", "Create Clothing LOD from Section"),
						LOCTEXT("MeshClickMenu_CreateClothingNewLod_ToolTip", "Create a clothing simulation mesh from the selected section and add it as a LOD to existing clothing data."),
						FNewToolMenuChoice(FNewMenuDelegate::CreateSP(Editor.Get(), &FSkeletalMeshEditor::FillCreateClothingLodMenu, LodIndex, SectionIndex)),
						FToolUIActionChoice(FUIAction(
							FExecuteAction(),
							FCanExecuteAction::CreateSP(Editor.Get(), &FSkeletalMeshEditor::CanCreateClothingLod, LodIndex, SectionIndex)
						)),
						EUserInterfaceActionType::Button
					);
				}
			}
		}));
		
		FToolMenuSection& LODSection = ViewportMenu->AddSection("LOD", LOCTEXT("MeshClickMenu_Section_LOD", "LOD"));
		LODSection.AddDynamicEntry("DynamicLODEntries", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
		{
			if (USkeletalMeshEditorContextMenuContext* MenuContext = InSection.FindContext<USkeletalMeshEditorContextMenuContext>())
			{
				TSharedPtr<FSkeletalMeshEditor> Editor = GetSkeletalMeshEditor(InSection.Context);
				if (Editor.IsValid())
				{
					const int32 LodIndex = MenuContext->LodIndex;
					const int32 SectionIndex = MenuContext->SectionIndex;

					if (Editor->SkeletalMesh != nullptr && Editor->SkeletalMesh->GetImportedModel()->LODModels.IsValidIndex(LodIndex))
					{
						const FSkeletalMeshLODInfo* SkeletalMeshLODInfo = Editor->SkeletalMesh->GetLODInfo(LodIndex);
						if (SkeletalMeshLODInfo != nullptr)
						{
							InSection.AddMenuEntry(
								"RemoveSectionFromLodAndBelow",
								FText::Format(LOCTEXT("MeshClickMenu_RemoveSectionFromLodAndBelow", "Generate section {1} up to LOD {0}"), LodIndex, SectionIndex),
								FText::Format(LOCTEXT("MeshClickMenu_RemoveSectionFromLodAndBelow_Tooltip", "Generated LODs will use section {1} up to LOD {0}, and ignore it for lower quality LODs"), LodIndex, SectionIndex),
								FSlateIcon(),
								FToolUIActionChoice(FUIAction(
									FExecuteAction::CreateSP(Editor.Get(), &FSkeletalMeshEditor::OnRemoveSectionFromLodAndBelowMenuItemClicked, LodIndex, SectionIndex)
								))
							);
						}
					}
				}
			}
		}));
	}
}

void FSkeletalMeshEditor::BakeMaterials()
{
	if (GetPersonaToolkit()->GetPreviewMeshComponent() != nullptr)
	{
		const IMeshMergeModule& Module = FModuleManager::Get().LoadModuleChecked<IMeshMergeModule>("MeshMergeUtilities");
		Module.GetUtilities().BakeMaterialsForComponent(GetPersonaToolkit()->GetPreviewMeshComponent());
	}
}

void FSkeletalMeshEditor::HandleObjectsSelected(const TArray<UObject*>& InObjects)
{
	if (DetailsView.IsValid())
	{
		DetailsView->SetObjects(InObjects);
	}
}

void FSkeletalMeshEditor::HandleObjectSelected(UObject* InObject)
{
	if (DetailsView.IsValid())
	{
		DetailsView->SetObject(InObject);
	}
}

void FSkeletalMeshEditor::HandleSelectionChanged(const TArrayView<TSharedPtr<ISkeletonTreeItem>>& InSelectedItems, ESelectInfo::Type InSelectInfo)
{
	RouteTreeSelectionToModeManager(InSelectedItems);

	if (DetailsView.IsValid())
	{
		TArray<UObject*> Objects;
		Algo::TransformIf(InSelectedItems, Objects, [](const TSharedPtr<ISkeletonTreeItem>& InItem) { return InItem->GetObject() != nullptr; }, [](const TSharedPtr<ISkeletonTreeItem>& InItem) { return InItem->GetObject(); });
		DetailsView->SetObjects(Objects);

		if (Binding.IsValid())
		{
			TArray<FName> BoneSelection;
			Algo::TransformIf(InSelectedItems, BoneSelection,
				[](const TSharedPtr<ISkeletonTreeItem>& InItem)
				{
					const bool bIsBoneType = InItem->IsOfTypeByName("FSkeletonTreeBoneItem");
					const bool bIsNotNone = InItem->GetRowItemName() != NAME_None;
					return bIsBoneType && bIsNotNone;
				},
				[](const TSharedPtr<ISkeletonTreeItem>& InItem){ return InItem->GetRowItemName(); });
			Binding->GetNotifier()->Notify(BoneSelection, ESkeletalMeshNotifyType::BonesSelected);
		}
	}
}

void FSkeletalMeshEditor::PostUndo(bool bSuccess)
{
	OnPostUndo.Broadcast();
}

void FSkeletalMeshEditor::PostRedo(bool bSuccess)
{
	OnPostUndo.Broadcast();
}

void FSkeletalMeshEditor::Tick(float DeltaTime)
{
	GetPersonaToolkit()->GetPreviewScene()->InvalidateViews();
}

TStatId FSkeletalMeshEditor::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FSkeletalMeshEditor, STATGROUP_Tickables);
}

void FSkeletalMeshEditor::HandleDetailsCreated(const TSharedRef<IDetailsView>& InDetailsView)
{
	DetailsView = InDetailsView;
}

void FSkeletalMeshEditor::HandleMeshDetailsCreated(const TSharedRef<IDetailsView>& InDetailsView)
{
	FPersonaModule& PersonaModule = FModuleManager::GetModuleChecked<FPersonaModule>("Persona");
	PersonaModule.CustomizeMeshDetails(InDetailsView, GetPersonaToolkit());
}

void FSkeletalMeshEditor::HandleViewportCreated(const TSharedRef<IPersonaViewport>& InViewport)
{
	Viewport = InViewport;
	FEditorViewportClient& ViewportClient = InViewport->GetViewportClient();

	// register callbacks to allow the asset to store the Bone Size viewport setting
	if (FAnimationViewportClient* AnimViewportClient = static_cast<FAnimationViewportClient*>(&ViewportClient))
	{
		AnimViewportClient->OnSetBoneSize.BindLambda([InSkeletalMesh=SkeletalMesh](float InBoneSize)
		{
			if (InSkeletalMesh)
			{
				InSkeletalMesh->Modify();
				InSkeletalMesh->SetBoneDrawSize(InBoneSize);
			}
		});
		
		AnimViewportClient->OnGetBoneSize.BindLambda([InSkeletalMesh=SkeletalMesh]() -> float
		{
			if (InSkeletalMesh)
			{
				return InSkeletalMesh->GetBoneDrawSize();
			}

			return 1.0f;
		});
	}
	
	// we need the viewport client to start out focused, or else it won't get ticked until we click inside it.
	ViewportClient.ReceivedFocus(ViewportClient.Viewport);
}

UObject* FSkeletalMeshEditor::HandleGetAsset()
{
	return GetEditingObject();
}

TFuture<bool> FSkeletalMeshEditor::HandleReimportMeshInternal(const FReimportParameters& ReimportParameters)
{
	TSharedPtr<TPromise<bool>> Promise = MakeShared<TPromise<bool>>();

	//The reimport will be asynchronous only if the reimport manager use Interchange.
	constexpr bool bAutomatedFalse = false;
	UE::Interchange::FAssetImportResultRef Result = FReimportManager::Instance()->ReimportAsync(SkeletalMesh, true, true, TEXT(""), nullptr, ReimportParameters.SourceFileIndex, ReimportParameters.bWithNewFile, bAutomatedFalse, ReimportParameters.bReimportWithDialog);

	Result->OnDone([Promise, SkeletonTreePtr = SkeletonTree, WeakSkeletalMesh = TWeakObjectPtr<USkeletalMesh>(SkeletalMesh)](UE::Interchange::FImportResult& Result)
		{
			auto ResetComponent = [Promise, SkeletonTreePtr, WeakSkeletalMesh, &Result]()
				{
					// Refresh skeleton tree
					SkeletonTreePtr->Refresh();

					const TArray<UInterchangeResult*>& Results = Result.GetResults()->GetResults();
					for (const UInterchangeResult* InterchangeResult : Results)
					{
						if (InterchangeResult->IsA<UInterchangeResultError_ReimportFail>())
						{
							Promise->SetValue(false);
							return;
						}
					}
					Promise->SetValue(true);
				};

			if (IsInGameThread())
			{
				ResetComponent();
			}
			else
			{
				Async(EAsyncExecution::TaskGraphMainThread, MoveTemp(ResetComponent));
			}
		});
	return Promise->GetFuture();
}

void FSkeletalMeshEditor::HandleReimportMesh(const FReimportParameters ReimportParameters)
{
	TSharedPtr<FScopedSuspendAlternateSkinWeightPreview> ScopedSuspendAlternateSkinnWeightPreview = MakeShared<FScopedSuspendAlternateSkinWeightPreview>(SkeletalMesh);
	TSharedPtr<FScopedSkeletalMeshReregisterContexts> ScopedReregisterComponents = MakeShared<FScopedSkeletalMeshReregisterContexts>(SkeletalMesh);
	HandleReimportMeshInternal(ReimportParameters).Then([ScopedSuspendAlternateSkinnWeightPreview, ScopedReregisterComponents](TFuture<bool> ReimportResult) {});
}

void FSkeletalMeshEditor::HandlePostReimport(UObject* InObject, bool bSuccess)
{
	// OnPostReimport is a global broadcast for every reimport in the editor session —
	// only react if it's the asset this editor instance is currently editing.
	if (!bSuccess || InObject != SkeletalMesh)
	{
		return;
	}

	if (SkeletonTree.IsValid())
	{
		SkeletonTree->DeselectAll();
		SkeletonTree->Refresh();
	}
}

TFuture<bool> ReimportLodInChain(USkeletalMesh* SkeletalMesh
	, UDebugSkelMeshComponent* PreviewMeshComponent
	, bool bWithNewFile
	, TSharedPtr<TArray<bool>> Dependencies
	, int32 LodIndex)
{
	TSharedPtr<TPromise<bool>> Promise = MakeShared<TPromise<bool>>();

	if (SkeletalMesh->GetLODNum() <= LodIndex)
	{
		//Nothing to do
		Promise->SetValue(false);
		return Promise->GetFuture();
	}
	
	TArray<bool>& DependenciesRef = *Dependencies.Get();

	//Do not reimport LOD that was re-import with the base mesh
	if (!SkeletalMesh->GetLODInfo(LodIndex)->bImportWithBaseMesh)
	{
		if (SkeletalMesh->GetLODInfo(LodIndex)->bHasBeenSimplified == false)
		{
			FString SourceFilenameBackup = SkeletalMesh->GetLODInfo(LodIndex)->SourceImportFilename;
			if (bWithNewFile)
			{
				SkeletalMesh->GetLODInfo(LodIndex)->SourceImportFilename.Empty();
			}

			FbxMeshUtils::ImportMeshLODDialog(SkeletalMesh, LodIndex, false).Then([Promise, SkeletalMesh, bWithNewFile, LodIndex, SourceFilenameBackup, PreviewMeshComponent, Dependencies](TFuture<bool> FutureResult)
				{
					TArray<bool>& DependenciesRef = *Dependencies.Get();
					const bool bResult = FutureResult.Get();
					if (!bResult)
					{
						if (bWithNewFile)
						{
							SkeletalMesh->GetLODInfo(LodIndex)->SourceImportFilename = SourceFilenameBackup;
						}
					}
					else
					{
						DependenciesRef[LodIndex] = true;
					}

					//Iterate the next lod chain
					if (SkeletalMesh->GetLODNum() > LodIndex + 1)
					{
						ReimportLodInChain(SkeletalMesh, PreviewMeshComponent, bWithNewFile, Dependencies, LodIndex + 1).Then([Promise](TFuture<bool> ReimportLodInChainFutureResult)
							{
								const bool bReimportLodInChainResult = ReimportLodInChainFutureResult.Get();
								Promise->SetValue(bReimportLodInChainResult);
							});
					}
					else
					{
						Promise->SetValue(bResult);
					}
				});
			return Promise->GetFuture();
		}
		else if (DependenciesRef[SkeletalMesh->GetLODInfo(LodIndex)->ReductionSettings.BaseLOD])
		{
			//Regenerate the LOD
			FSkeletalMeshUpdateContext UpdateContext;
			UpdateContext.SkeletalMesh = SkeletalMesh;
			UpdateContext.AssociatedComponents.Push(PreviewMeshComponent);
			FLODUtilities::SimplifySkeletalMeshLOD(UpdateContext, LodIndex, GetTargetPlatformManagerRef().GetRunningTargetPlatform(), false);
			DependenciesRef[LodIndex] = true;
		}
	}
	//Iterate the next Lod in the chain
	if (SkeletalMesh->GetLODNum() > LodIndex + 1)
	{
		ReimportLodInChain(SkeletalMesh, PreviewMeshComponent, bWithNewFile, Dependencies, LodIndex + 1).Then([Promise](TFuture<bool> ReimportLodInChainFutureResult)
			{
				const bool bReimportLodInChainResult = ReimportLodInChainFutureResult.Get();
				Promise->SetValue(bReimportLodInChainResult);
			});
	}
	else
	{
		//We have iterate all LodIndex set the promise
		Promise->SetValue(true);
	}
	return Promise->GetFuture();
}

TFuture<bool> ReimportAllCustomLODs(USkeletalMesh* SkeletalMesh, UDebugSkelMeshComponent* PreviewMeshComponent, bool bWithNewFile)
{
	TSharedPtr<TPromise<bool>> Promise = MakeShared<TPromise<bool>>();
	//Find the dependencies of the generated LOD
	TSharedPtr<TArray<bool>> Dependencies = MakeShared<TArray<bool>>();
	Dependencies->AddZeroed(SkeletalMesh->GetLODNum());

	int32 LodIndex = 1;
	ReimportLodInChain(SkeletalMesh, PreviewMeshComponent, bWithNewFile, Dependencies, LodIndex).Then([Promise](TFuture<bool> FutureResult)
		{
			bool bResult = FutureResult.Get();
			Promise->SetValue(bResult);
		});
	return Promise->GetFuture();
}

void FSkeletalMeshEditor::HandleReimportAllMeshInternal(const FReimportParameters& ReimportParameters)
{
	// Reimport the asset
	if (SkeletalMesh)
	{
		TSharedPtr<FScopedSuspendAlternateSkinWeightPreview> ScopedSuspendAlternateSkinnWeightPreview = MakeShared<FScopedSuspendAlternateSkinWeightPreview>(SkeletalMesh);
		TSharedPtr<FScopedSkeletalMeshReregisterContexts> ScopedReregisterComponents = MakeShared<FScopedSkeletalMeshReregisterContexts>(SkeletalMesh);
		//Reimport base LOD
		HandleReimportMeshInternal(ReimportParameters).Then([this, ReimportParameters, ScopedSuspendAlternateSkinnWeightPreview, ScopedReregisterComponents](TFuture<bool> Result)
		{
			check(IsInGameThread());
			//import all custom LODs
			if (Result.Get() && SkeletalMesh->GetLODNum() > 1)
			{
				ReimportAllCustomLODs(SkeletalMesh.Get(), GetPersonaToolkit()->GetPreviewMeshComponent(), ReimportParameters.bWithNewFile);
			}
		});
	}
}

void FSkeletalMeshEditor::HandleReimportAllMesh(const FReimportParameters ReimportParameters)
{
	HandleReimportAllMeshInternal(ReimportParameters);
}


void FSkeletalMeshEditor::HandleOnPreviewSceneSettingsCustomized(IDetailLayoutBuilder& DetailBuilder)
{
	DetailBuilder.HideCategory("Mesh");
	DetailBuilder.HideCategory("Physics");
	// in mesh editor, we hide preview mesh section and additional mesh section
	// sometimes additional meshes are interfering with preview mesh, it is not a great experience
	DetailBuilder.HideCategory("Additional Meshes");
}

bool FSkeletalMeshEditor::IsMeshSectionSelectionChecked() const
{
	return GetPersonaToolkit()->GetPreviewScene()->AllowMeshHitProxies();
}

void FSkeletalMeshEditor::HandleMeshClick(HActor* HitProxy, const FViewportClick& Click)
{
	USkeletalMeshComponent* Component = GetPersonaToolkit()->GetPreviewMeshComponent();
	if (Component)
	{
		Component->SetSelectedEditorSection(HitProxy->SectionIndex);
		Component->PushSelectionToProxy();

		if(Click.GetKey() == EKeys::RightMouseButton)
		{
			USkeletalMeshEditorContextMenuContext* MenuContext = NewObject<USkeletalMeshEditorContextMenuContext>();
			MenuContext->LodIndex = Component->GetPredictedLODLevel();
			MenuContext->SectionIndex = HitProxy->SectionIndex;

			FToolMenuContext ToolMenuContext(MenuContext);
			InitToolMenuContext(ToolMenuContext);

			FSlateApplication::Get().PushMenu(
				FSlateApplication::Get().GetActiveTopLevelWindow().ToSharedRef(),
				FWidgetPath(),
				UToolMenus::Get()->GenerateWidget(ViewportMenuId, ToolMenuContext),
				FSlateApplication::Get().GetCursorPos(),
				FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
				);
		}
	}
}

TSharedPtr<ISkeletalMeshEditorBinding> FSkeletalMeshEditor::GetBinding()
{
	if (!Binding)
	{
		Binding = MakeShared<FSkeletalMeshEditorBinding>(SharedThis(this));
	}
	
	return Binding;
}

FSimpleMulticastDelegate& FSkeletalMeshEditor::OnPreSaveAsset()
{
	return OnPreSaveAssetDelegate;
}

FSimpleMulticastDelegate& FSkeletalMeshEditor::OnPreSaveAssetAs()
{
	return OnPreSaveAssetAsDelegate;
}

TSharedPtr<FUICommandInfo> FSkeletalMeshEditor::GetResetBoneTransformsCommand()
{
	const FSkeletalMeshEditorCommands Commands = FSkeletalMeshEditorCommands::Get();
	return Commands.ResetBoneTransforms;
}

TSharedPtr<FUICommandInfo> FSkeletalMeshEditor::GetResetAllBonesTransformsCommand()
{
	const FSkeletalMeshEditorCommands Commands = FSkeletalMeshEditorCommands::Get();
	return Commands.ResetAllBonesTransforms;
}

FSkeletalMeshEditorBinding::FSkeletalMeshEditorBinding(TSharedRef<FSkeletalMeshEditor> InEditor)
	: Editor(InEditor)
{}

TSharedPtr<ISkeletalMeshNotifier> FSkeletalMeshEditorBinding::GetNotifier()
{
	if (!Notifier)
	{
		Notifier = MakeShared<FSkeletalMeshEditorNotifier>(Editor.Pin().ToSharedRef());
	}
	return Notifier;	
}

ISkeletalMeshEditorBinding::NameFunction FSkeletalMeshEditorBinding::GetNameFunction()
{
	return [this](HHitProxy* InHitProxy) -> TOptional<FName>
	{
		if (const HPersonaBoneHitProxy* BoneHitProxy = HitProxyCast<HPersonaBoneHitProxy>(InHitProxy))
		{
			return BoneHitProxy->BoneName;
		}

		static const TOptional<FName> Dummy;
		return Dummy;
	};
}

TArray<FName> FSkeletalMeshEditorBinding::GetSelectedBones() const
{
	TArray<FName> Selection;

	if (!Editor.IsValid())
	{
		return Selection;
	}

	const TArray<TSharedPtr<ISkeletonTreeItem>> SelectedItems = Editor.Pin()->GetSkeletonTree()->GetSelectedItems();

	Algo::TransformIf(SelectedItems, Selection,
	[](const TSharedPtr<ISkeletonTreeItem>& InItem) { return InItem->IsOfTypeByName("FSkeletonTreeBoneItem") && InItem->GetRowItemName() != NAME_None; },
	[](const TSharedPtr<ISkeletonTreeItem>& InItem) { return InItem->GetRowItemName(); });
	
	return Selection;
}

FSkeletalMeshEditorNotifier::FSkeletalMeshEditorNotifier(TSharedRef<FSkeletalMeshEditor> InEditor)
	: ISkeletalMeshNotifier()
	, Editor(InEditor)
{}

void FSkeletalMeshEditorNotifier::HandleNotification(const TArray<FName>& BoneNames, const ESkeletalMeshNotifyType InNotifyType)
{
	if (Notifying() || !Editor.IsValid())
	{
		return;
	}

	TSharedRef<ISkeletonTree> SkeletonTree = Editor.Pin()->GetSkeletonTree();

	switch (InNotifyType)
	{
	case ESkeletalMeshNotifyType::BonesAdded:
		SkeletonTree->DeselectAll();
		SkeletonTree->Refresh();
		break;
	case ESkeletalMeshNotifyType::BonesRemoved:
		SkeletonTree->DeselectAll();
		SkeletonTree->Refresh();
		break;
	case ESkeletalMeshNotifyType::BonesMoved:
		break;
	case ESkeletalMeshNotifyType::BonesSelected:
		SkeletonTree->DeselectAll();
		if (!BoneNames.IsEmpty())
		{
			for (const FName BoneName: BoneNames)
			{
				SkeletonTree->SetSelectedBone(BoneName, ESelectInfo::Direct);
			}
		}
		break;
	case ESkeletalMeshNotifyType::BonesRenamed:
		SkeletonTree->DeselectAll();
		SkeletonTree->Refresh();
		if (!BoneNames.IsEmpty())
		{
			SkeletonTree->SetSelectedBone(BoneNames[0], ESelectInfo::Direct);
		}
		break;
	case ESkeletalMeshNotifyType::HierarchyChanged:
		SkeletonTree->DeselectAll();
		SkeletonTree->Refresh();
		break;
	}
}

#undef LOCTEXT_NAMESPACE
