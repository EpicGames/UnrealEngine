// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "UObject/GCObject.h"
#include "TickableEditorObject.h"
#include "EditorUndoClient.h"
#include "Toolkits/IToolkitHost.h"
#include "ISkeletalMeshEditor.h"
#include "SkeletalMeshNotifier.h"
#include "Containers/ArrayView.h"
#include "Async/Future.h"

class FSkeletalMeshEditorModeUILayer;
class IDetailLayoutBuilder;
class IDetailsView;
class IPersonaToolkit;
class IPersonaViewport;
class ISkeletonTree;
class USkeletalMesh;
class UClothingAssetBase;
class ISkeletonTreeItem;
struct HActor;
struct FViewportClick;
struct FSkeletalMeshClothBuildParams;
struct FToolMenuContext;
class UToolMenu;
class SSkeletalMeshEditorToolbox;
class FSkeletalMeshEditorBinding;

namespace SkeletalMeshEditorModes
{
	// Mode identifiers
	extern const FName SkeletalMeshEditorMode;
}

namespace SkeletalMeshEditorTabs
{
	// Tab identifiers
	extern const FName DetailsTab;
	extern const FName SkeletonTreeTab;
	extern const FName ViewportTab;
	extern const FName AdvancedPreviewTab;
	extern const FName AssetDetailsTab;
	extern const FName MorphTargetsTab;
	extern const FName MeshDetailsTab;
	extern const FName AnimationMappingTab;
	extern const FName CurveMetadataTab;
	extern const FName FindReplaceTab;
}

class FSkeletalMeshEditor : public ISkeletalMeshEditor, public FGCObject, public FEditorUndoClient, public FTickableEditorObject
{
public:
	FSkeletalMeshEditor();

	virtual ~FSkeletalMeshEditor();

	/** Edits the specified Skeleton object */
	void InitSkeletalMeshEditor(const EToolkitMode::Type Mode, const TSharedPtr<class IToolkitHost>& InitToolkitHost, class USkeletalMesh* InSkeletalMesh);

	// FPersonaAssetEditorToolkit
	virtual void OnToolkitHostingStarted(const TSharedRef<IToolkit>& Toolkit) override;
	virtual void OnToolkitHostingFinished(const TSharedRef<IToolkit>& Toolkit) override;
	virtual void PostInitAssetEditor() override;

	/** IHasPersonaToolkit interface */
	virtual TSharedRef<class IPersonaToolkit> GetPersonaToolkit() const override { return PersonaToolkit.ToSharedRef(); }

	/** IToolkit interface */
	virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;
	virtual void InitToolMenuContext(FToolMenuContext& MenuContext) override;

	virtual void AddViewportOverlayWidget(TSharedRef<SWidget> InOverlaidWidget, int32 ZOrder = INDEX_NONE) override;
	virtual void RemoveViewportOverlayWidget(TSharedRef<SWidget> InOverlaidWidget) override;
	
	/** FBaseToolkit overrides */
	virtual bool ProcessCommandBindings(const FKeyEvent& InKeyEvent) const override;
	
	//~ Begin FAssetEditorToolkit Interface.
	virtual bool OnRequestClose(EAssetEditorCloseReason InCloseReason) override;
	virtual void SaveAsset_Execute() override;
	virtual void SaveAssetAs_Execute() override;
	//~ End FAssetEditorToolkit Interface.

	/** FEditorUndoClient interface */
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;

	/** FTickableEditorObject Interface */
	virtual void Tick(float DeltaTime) override;
	virtual TStatId GetStatId() const override;
	virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Always; }

	/** @return the documentation location for this editor */
	virtual FString GetDocumentationLink() const override
	{
		return FString(TEXT("AnimatingObjects/SkeletalMeshAnimation/Persona/Modes/Mesh"));
	}

	/** FGCObject interface */
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FSkeletalMeshEditor");
	}

	/** Get the skeleton tree widget */
	TSharedRef<class ISkeletonTree> GetSkeletonTree() const { return SkeletonTree.ToSharedRef(); }

	/** Get the MorphTargetViewer widget */
	TSharedRef<class IMorphTargetViewer> GetMorphTargetViewer() const {return MorphTargetViewer.ToSharedRef(); }

	void HandleDetailsCreated(const TSharedRef<class IDetailsView>& InDetailsView);

	void HandleMeshDetailsCreated(const TSharedRef<class IDetailsView>& InDetailsView);

	void HandleViewportCreated(const TSharedRef<class IPersonaViewport>& InViewport);

	UObject* HandleGetAsset();

	void HandleObjectsSelected(const TArray<UObject*>& InObjects);

	// Returns the currently hosted toolkit. Can be invalid if no toolkit is being hosted.
	TSharedPtr<IToolkit> GetHostedToolkit() const { return HostedToolkit; }
	
	virtual TSharedPtr<ISkeletalMeshEditorBinding> GetBinding() override;
	virtual FSimpleMulticastDelegate& OnPreSaveAsset() override;
	virtual FSimpleMulticastDelegate& OnPreSaveAssetAs() override;
	virtual TSharedPtr<FUICommandInfo> GetResetBoneTransformsCommand() override;
	virtual TSharedPtr<FUICommandInfo> GetResetAllBonesTransformsCommand() override;
	
protected:
	void OnBuildViewportInteractions(UViewportInteractionsBehaviorSource* Source) override;
	
private:
	void HandleObjectSelected(UObject* InObject);
	
	void HandleSelectionChanged(const TArrayView<TSharedPtr<ISkeletonTreeItem>>& InSelectedItems, ESelectInfo::Type InSelectInfo);

	struct FReimportParameters
	{
		FReimportParameters(int32 InSourceFileIndex, bool bInWithNewFile, bool bInReimportWithDialog)
			: SourceFileIndex(InSourceFileIndex)
			, bWithNewFile(bInWithNewFile)
			, bReimportWithDialog(bInReimportWithDialog)
		{}

		int32 SourceFileIndex = INDEX_NONE;
		bool bWithNewFile = false;
		bool bReimportWithDialog = false;
	};

	void HandleReimportMesh(const FReimportParameters ReimportParameters);
	TFuture<bool> HandleReimportMeshInternal(const FReimportParameters& ReimportParameters);

	void HandleReimportAllMesh(const FReimportParameters ReimportParameters);
	void HandleReimportAllMeshInternal(const FReimportParameters& ReimportParameters);

	// Refreshes the skeleton tree when our edited mesh is re-imported outside of this editor's
	// own re-import button (Content Browser, scripted, drag-drop, etc.). The in-editor button
	// has its own refresh path inside HandleReimportMeshInternal's OnDone callback.
	void HandlePostReimport(UObject* InObject, bool bSuccess);

	void HandleOnPreviewSceneSettingsCustomized(IDetailLayoutBuilder& DetailBuilder);

	/** Callback for checking whether the UV drawing is switched on. */
	bool IsMeshSectionSelectionChecked() const;

	void HandleMeshClick(HActor* HitProxy, const FViewportClick& Click);

	// Clothing menu handlers (builds and handles clothing context menu options)
	void FillApplyClothingAssetMenu(FMenuBuilder& MenuBuilder, int32 InLodIndex, int32 InSectionIndex);
	void FillCreateClothingMenu(FMenuBuilder& MenuBuilder, int32 InLodIndex, int32 InSectionIndex);
	void FillCreateClothingLodMenu(FMenuBuilder& MenuBuilder, int32 InLodIndex, int32 InSectionIndex);
	void OnRemoveClothingAssetMenuItemClicked(int32 InLodIndex, int32 InSectionIndex);
	void OnCreateClothingAssetMenuItemClicked(FSkeletalMeshClothBuildParams& Params);
	void OnApplyClothingAssetClicked(UClothingAssetBase* InAssetToApply, int32 InMeshLodIndex, int32 InMeshSectionIndex, int32 InClothLodIndex);

	bool CanApplyClothing(int32 InLodIndex, int32 InSectionIndex);
	bool CanRemoveClothing(int32 InLodIndex, int32 InSectionIndex);
	bool CanCreateClothing(int32 InLodIndex, int32 InSectionIndex);
	bool CanCreateClothingLod(int32 InLodIndex, int32 InSectionIndex);

	void ApplyClothing(UClothingAssetBase* InAsset, int32 InLodIndex, int32 InSectionIndex, int32 InClothingLod);
	void RemoveClothing(int32 InLodIndex, int32 InSectionIndex);
	//////////////////////////////////////////////////////////////////////////

	// Generate LOD sections menu handlers
	void OnRemoveSectionFromLodAndBelowMenuItemClicked(int32 LodIndex, int32 SectionIndex);
	//////////////////////////////////////////////////////////////////////////

	void RegisterReimportContextMenu(const FName InBaseMenuName, bool bWithDialog);

	static TSharedPtr<FSkeletalMeshEditor> GetSkeletalMeshEditor(const FToolMenuContext& InMenuContext);

private:
	void ExtendMenu();

	void BakeMaterials();
	void ExtendToolbar();

	void BindCommands();

public:
	/** Multicast delegate fired on global undo/redo */
	FSimpleMulticastDelegate OnPostUndo;

	/** Multicast delegate fired on curves changing */
	FSimpleMulticastDelegate OnCurvesChanged;

private:
	/** The skeleton we are editing */
	TObjectPtr<USkeletalMesh> SkeletalMesh;

	/** Toolbar extender */
	TSharedPtr<FExtender> ToolbarExtender;

	/** Menu extender */
	TSharedPtr<FExtender> MenuExtender;

	/** Persona toolkit */
	TSharedPtr<class IPersonaToolkit> PersonaToolkit;

	/** Skeleton tree */
	TSharedPtr<class ISkeletonTree> SkeletonTree;

	/** Morph target viewer */
	TSharedPtr<class IMorphTargetViewer> MorphTargetViewer;

	/** Viewport */
	TSharedPtr<class IPersonaViewport> Viewport;

	/** Details panel */
	TSharedPtr<class IDetailsView> DetailsView;

	// The toolkit we're currently hosting.
	TSharedPtr<IToolkit> HostedToolkit;

	// This is set in ModeUILayer to be the menu category where new tabs are registered to be enabled by the user
	TSharedPtr<FWorkspaceItem> SkeletalMeshEditorMenuCategory;
	
	TSharedPtr<FSkeletalMeshEditorModeUILayer> ModeUILayer;

	// The toolbox widget
	TSharedPtr<SSkeletalMeshEditorToolbox> ToolboxWidget;

	// Binding to send/receive skeletal mesh modifications
	TSharedPtr<FSkeletalMeshEditorBinding> Binding;

	FSimpleMulticastDelegate OnPreSaveAssetDelegate;
	FSimpleMulticastDelegate OnPreSaveAssetAsDelegate;
};

/**
 * FSkeletalMeshEditorNotifier
 */

class FSkeletalMeshEditorNotifier: public ISkeletalMeshNotifier
{
public:
	FSkeletalMeshEditorNotifier(TSharedRef<FSkeletalMeshEditor> InEditor);
	virtual void HandleNotification(const TArray<FName>& BoneNames, const ESkeletalMeshNotifyType InNotifyType) override;
	
private:
	TWeakPtr<FSkeletalMeshEditor> Editor;
};

/**
 * FSkeletalMeshEditorBinding
 */

class FSkeletalMeshEditorBinding: public ISkeletalMeshEditorBinding
{
public:
	FSkeletalMeshEditorBinding(TSharedRef<FSkeletalMeshEditor> InEditor);

	virtual TSharedPtr<ISkeletalMeshNotifier> GetNotifier() override;
	virtual NameFunction GetNameFunction() override;
	virtual TArray<FName> GetSelectedBones() const override;

private:
	TWeakPtr<FSkeletalMeshEditor> Editor;
	TSharedPtr<FSkeletalMeshEditorNotifier> Notifier;
};
