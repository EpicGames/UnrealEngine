// Copyright Epic Games, Inc. All Rights Reserved.


#include "SkeletalMeshEditorCommands.h"

#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"

#define LOCTEXT_NAMESPACE "SkeletalMeshEditorCommands"

void FSkeletalMeshEditorCommands::RegisterCommands()
{
	UI_COMMAND(ReimportMesh, "Reimport Mesh", "Reimport the base mesh.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ReimportMeshWithNewFile, "Reimport Mesh With New File", "Reimport the base mesh using a new source file.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ReimportAllMesh, "Reimport Mesh + LODs", "Reimport the base mesh and all the custom LODs.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ReimportAllMeshWithNewFile, "Reimport Mesh + LODs With New File", "Reimport the base mesh using a new source file and all the custom LODs (No new source file for LODs).", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(ReimportWithDialog, "Reimport Mesh With Dialog", "Reimport the base mesh with dialog.", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(BakeMaterials, "Bake Out Materials", "Bake out Materials for given LOD(s).", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(ResetBoneTransforms, "Reset Bone Transforms", "Reset selected bones transforms.", EUserInterfaceActionType::Button, FInputChord(EKeys::I, EModifierKey::Control) );
	UI_COMMAND(ResetAllBonesTransforms, "Reset All Bones Transforms", "Reset all bones transforms.", EUserInterfaceActionType::Button, FInputChord(EKeys::I, EModifierKey::Control | EModifierKey::Shift) );
}

#undef LOCTEXT_NAMESPACE
