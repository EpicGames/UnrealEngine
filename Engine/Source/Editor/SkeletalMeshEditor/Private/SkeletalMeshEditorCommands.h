// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class FUICommandInfo;

class FSkeletalMeshEditorCommands : public TCommands<FSkeletalMeshEditorCommands>
{
public:
	FSkeletalMeshEditorCommands()
		: TCommands<FSkeletalMeshEditorCommands>(TEXT("SkeletalMeshEditor"), NSLOCTEXT("Contexts", "SkeletalMeshEditor", "Skeletal Mesh Editor"), NAME_None, FAppStyle::GetAppStyleSetName())
	{
	}

	virtual void RegisterCommands() override;

public:
	// reimport
	TSharedPtr<FUICommandInfo> ReimportMesh;
	TSharedPtr<FUICommandInfo> ReimportMeshWithNewFile;
	TSharedPtr<FUICommandInfo> ReimportAllMesh;
	TSharedPtr<FUICommandInfo> ReimportAllMeshWithNewFile;

	// reimport with dialog
	TSharedPtr<FUICommandInfo> ReimportWithDialog;
	
	// bake materials for this skeletal mesh
	TSharedPtr<FUICommandInfo> BakeMaterials;

	// Reset bones transforms
	TSharedPtr< FUICommandInfo > ResetBoneTransforms;
	TSharedPtr< FUICommandInfo > ResetAllBonesTransforms;
};
