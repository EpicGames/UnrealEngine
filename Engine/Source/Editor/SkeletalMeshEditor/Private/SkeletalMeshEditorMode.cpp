// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletalMeshEditorMode.h"

#include "PersonaModule.h"
#include "SkeletalMeshEditor.h"
#include "ISkeletonTree.h"
#include "IMorphTargetViewer.h"
#include "ISkeletonEditorModule.h"
#include "IPersonaToolkit.h"
#include "SControlRigMappingWindow.h"
#include "Styling/AppStyle.h"
#include "Engine/SkeletalMesh.h"
#include "AnimAssetFindReplace.h"
#include "ISkeletalMeshEditorModule.h"
#include "PersonaTabs.h"

#define LOCTEXT_NAMESPACE "SkeletalMeshEditorMode"

FSkeletalMeshEditorMode::FSkeletalMeshEditorMode(
	TSharedRef<FWorkflowCentricApplication> InHostingApp,
	TSharedRef<ISkeletonTree> InSkeletonTree,
	TSharedRef<IMorphTargetViewer> InMorphTargetViewer
	)
	: FApplicationMode(SkeletalMeshEditorModes::SkeletalMeshEditorMode)
{
	FWorkflowAllowedTabSet& TabFactories = ApplicationModeTabFactories;

	TSharedRef<FSkeletalMeshEditor> SkeletalMeshEditor = StaticCastSharedRef<FSkeletalMeshEditor>(InHostingApp);

	ISkeletonEditorModule& SkeletonEditorModule = FModuleManager::LoadModuleChecked<ISkeletonEditorModule>("SkeletonEditor");
	TabFactories.RegisterFactory(SkeletonEditorModule.CreateSkeletonTreeTabFactory(InHostingApp, InSkeletonTree));

	FOnObjectsSelected OnObjectsSelected = FOnObjectsSelected::CreateSP(&SkeletalMeshEditor.Get(), &FSkeletalMeshEditor::HandleObjectsSelected);

	FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");
	TabFactories.RegisterFactory(PersonaModule.CreateDetailsTabFactory(InHostingApp, FOnDetailsCreated::CreateSP(&SkeletalMeshEditor.Get(), &FSkeletalMeshEditor::HandleDetailsCreated)));

	FPersonaViewportArgs ViewportArgs(SkeletalMeshEditor->GetPersonaToolkit()->GetPreviewScene());
	ViewportArgs.ContextName = TEXT("SkeletalMeshEditor.Viewport");
	ViewportArgs.OnViewportCreated = FOnViewportCreated::CreateSP(SkeletalMeshEditor, &FSkeletalMeshEditor::HandleViewportCreated);

	PersonaModule.RegisterPersonaViewportTabFactories(TabFactories, InHostingApp, ViewportArgs);

	TabFactories.RegisterFactory(PersonaModule.CreateAdvancedPreviewSceneTabFactory(InHostingApp, SkeletalMeshEditor->GetPersonaToolkit()->GetPreviewScene()));
	TabFactories.RegisterFactory(PersonaModule.CreateAssetDetailsTabFactory(InHostingApp, FOnGetAsset::CreateSP(&SkeletalMeshEditor.Get(), &FSkeletalMeshEditor::HandleGetAsset), FOnDetailsCreated::CreateSP(&SkeletalMeshEditor.Get(), &FSkeletalMeshEditor::HandleMeshDetailsCreated)));
	TabFactories.RegisterFactory(PersonaModule.CreateMorphTargetTabFactory(InHostingApp, InMorphTargetViewer));
	TabFactories.RegisterFactory(PersonaModule.CreateCurveMetadataEditorTabFactory(InHostingApp, SkeletalMeshEditor->HandleGetAsset(), SkeletalMeshEditor->GetPersonaToolkit()->GetPreviewScene(), OnObjectsSelected));
	TabFactories.RegisterFactory(CreateMeshControllerMappingTabFactory(InHostingApp, Cast<USkeletalMesh> (SkeletalMeshEditor->HandleGetAsset()), SkeletalMeshEditor->OnPostUndo));
	TabFactories.RegisterFactory(PersonaModule.CreateAnimAssetFindReplaceTabFactory(InHostingApp, FAnimAssetFindReplaceConfig()));
	TabFactories.RegisterFactory(PersonaModule.CreatePersonaToolboxTabFactory(SkeletalMeshEditor));

	TabLayout = FTabManager::NewLayout("Standalone_SkeletalMeshEditor_Layout_v5")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()
				->SetSizeCoefficient(0.9f)
				->SetOrientation(Orient_Horizontal)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.2f)
					->SetHideTabWell(true)
					->AddTab(FPersonaTabs::ToolboxID, ETabState::OpenedTab)
					->SetForegroundTab(FPersonaTabs::ToolboxID)
				)
				->Split
				(
					FTabManager::NewSplitter()
					->SetSizeCoefficient(0.6f)
					->SetOrientation(Orient_Vertical)
					->Split
					(	
						FTabManager::NewStack()
						->SetSizeCoefficient(0.65f)
						->SetHideTabWell(true)
						->AddTab(SkeletalMeshEditorTabs::ViewportTab, ETabState::OpenedTab)
					)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.5f)
						->SetHideTabWell(false)
						->AddTab(SkeletalMeshEditorTabs::FindReplaceTab, ETabState::ClosedTab)
					)
				)
				->Split
				(
					FTabManager::NewSplitter()
					->SetSizeCoefficient(0.2f)
					->SetOrientation(Orient_Vertical)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.5f)
						->SetHideTabWell(false)
						->AddTab(SkeletalMeshEditorTabs::SkeletonTreeTab, ETabState::OpenedTab)
						->AddTab(SkeletalMeshEditorTabs::MorphTargetsTab, ETabState::OpenedTab)
						->AddTab(SkeletalMeshEditorTabs::CurveMetadataTab, ETabState::OpenedTab)
						->SetForegroundTab(SkeletalMeshEditorTabs::SkeletonTreeTab)
					)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.5f)
						->SetHideTabWell(false)
						->AddTab(SkeletalMeshEditorTabs::AssetDetailsTab, ETabState::OpenedTab)
						->AddTab(SkeletalMeshEditorTabs::DetailsTab, ETabState::OpenedTab)
						->AddTab(SkeletalMeshEditorTabs::AdvancedPreviewTab, ETabState::OpenedTab)
						->SetForegroundTab(SkeletalMeshEditorTabs::AssetDetailsTab)
					)
				)
			)
		);

	PersonaModule.OnRegisterTabs().Broadcast(TabFactories, InHostingApp);
	LayoutExtender = MakeShared<FLayoutExtender>();
	PersonaModule.OnRegisterLayoutExtensions().Broadcast(*LayoutExtender.Get());

	ISkeletalMeshEditorModule* Module = &FModuleManager::LoadModuleChecked<ISkeletalMeshEditorModule>("SkeletalMeshEditor");
	Module->OnRegisterLayoutExtensions().Broadcast(*LayoutExtender);
	
	TabLayout->ProcessExtensions(*LayoutExtender.Get());

	SkeletalMeshEditor->RegisterModeToolbarIfUnregistered(GetModeName());
}

void FSkeletalMeshEditorMode::RegisterTabFactories(TSharedPtr<FTabManager> InTabManager)
{
	if (TSharedPtr<FWorkflowCentricApplication> HostingApp = GetHost())
	{
		TSharedRef<FTabManager> TabManager = InTabManager.ToSharedRef();
		HostingApp->RegisterTabSpawners(TabManager);
		RegisterTabFactoriesWithAppAndManager(HostingApp.Get(), TabManager);
	}
}

TSharedRef<class FWorkflowTabFactory> FSkeletalMeshEditorMode::CreateMeshControllerMappingTabFactory(const TSharedRef<class FWorkflowCentricApplication>& InHostingApp, const TWeakObjectPtr<class USkeletalMesh>& InEditingMesh, FSimpleMulticastDelegate& OnPostUndo) const
{
	return MakeShareable(new FMeshControllerMappingTabSummoner(InHostingApp, InEditingMesh, OnPostUndo));
}

/////////////////////////////////////////////////////
// FAnimationMappingWindowTabSummoner
static const FName ControlRigMappingWindowID("ControlRigMappingWindow");

FMeshControllerMappingTabSummoner::FMeshControllerMappingTabSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp, const TWeakObjectPtr<class USkeletalMesh>& InEditingMesh, FSimpleMulticastDelegate& InOnPostUndo)
	: FWorkflowTabFactory(ControlRigMappingWindowID, InHostingApp)
	, SkeletalMesh(InEditingMesh)
	, OnPostUndo(InOnPostUndo)
{
	TabLabel = LOCTEXT("ControlRigMappingWindowTabTitle", "Control Rig");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.Tabs.ControlRigMappingWindow");

	EnableTabPadding();
	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("ControlRigMappingWindowTabView", "Control Rig");
	ViewMenuTooltip = LOCTEXT("ControlRigMappingWindowTabView_ToolTip", "Configure Animation Controller Settings");
}

TSharedRef<SWidget> FMeshControllerMappingTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return SNew(SControlRigMappingWindow, SkeletalMesh, OnPostUndo);
}

TSharedPtr<SToolTip> FMeshControllerMappingTabSummoner::CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const
{
	return  IDocumentation::Get()->CreateToolTip(LOCTEXT("ControlRigMappingWindowTooltip", "In this panel, you can add new animation controllers and configure settings"), NULL, TEXT("Shared/Editors/Persona"), TEXT("ControlRigMappingWindow"));
}

#undef LOCTEXT_NAMESPACE
