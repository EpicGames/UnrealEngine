// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "WorkflowOrientedApp/WorkflowCentricApplication.h"
#include "PersonaDelegates.h"
#include "WorkflowOrientedApp/ApplicationMode.h"
#include "IDocumentation.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"

class SToolTip;

class FSkeletalMeshEditorMode : public FApplicationMode
{
public:
	FSkeletalMeshEditorMode(
		TSharedRef<FWorkflowCentricApplication> InHostingApp,
		TSharedRef<class ISkeletonTree> InSkeletonTree,
		TSharedRef<class IMorphTargetViewer> InMorphTargetViewer
		);

	/** FApplicationMode interface */
	virtual void RegisterTabFactories(TSharedPtr<FTabManager> InTabManager) override;

protected:
	/** Mesh interface class */
	TSharedRef<class FWorkflowTabFactory> CreateMeshControllerMappingTabFactory(const TSharedRef<class FWorkflowCentricApplication>& InHostingApp, const TWeakObjectPtr<class USkeletalMesh>& InEditingMesh, FSimpleMulticastDelegate& OnPostUndo) const;
};

struct FMeshPropertiesSummoner : public FWorkflowTabFactory
{
	FMeshPropertiesSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp, FOnGetAsset InOnGetAsset);

	// FWorkflowTabFactory interface
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;

	// Create a tooltip widget for the tab
	virtual TSharedPtr<SToolTip> CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const override
	{
		return  IDocumentation::Get()->CreateToolTip(NSLOCTEXT("MeshProperties", "MeshDetailsTooltip", "The Mesh Details tab lets you edit properties (materials etc) of the current Skeletal Mesh."), NULL, TEXT("Shared/Editors/Persona"), TEXT("MeshDetail_Window"));
	}
	// FWorkflowTabFactory interface

private:
	FOnGetAsset OnGetAsset;
};

/////////////////////////////////////////////////////
// FAnimationMappingWindowTabSummoner

struct FMeshControllerMappingTabSummoner : public FWorkflowTabFactory
{
public:
	FMeshControllerMappingTabSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp, const TWeakObjectPtr<class USkeletalMesh>& InEditingMesh, FSimpleMulticastDelegate& InOnPostUndo);

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;

	// Create a tooltip widget for the tab
	virtual TSharedPtr<SToolTip> CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const override;

private:
	TWeakObjectPtr<class USkeletalMesh> SkeletalMesh;
	FSimpleMulticastDelegate& OnPostUndo;
};

