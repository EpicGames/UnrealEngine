// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletalMeshEditorModeUILayer.h"

#include "ISkeletalMeshEditorModule.h"
#include "SkeletalMeshEditor.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"
#include "Modules/ModuleManager.h"

void USkeletalMeshEditorUISubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	ISkeletalMeshEditorModule& SkeletalMeshEditorModule = FModuleManager::GetModuleChecked<ISkeletalMeshEditorModule>("SkeletalMeshEditor");
	SkeletalMeshEditorModule.OnRegisterLayoutExtensions().AddUObject(this, &USkeletalMeshEditorUISubsystem::RegisterLayoutExtensions);
}

void USkeletalMeshEditorUISubsystem::Deinitialize()
{
	ISkeletalMeshEditorModule& SkeletalMeshEditorModule = FModuleManager::GetModuleChecked<ISkeletalMeshEditorModule>("SkeletalMeshEditor");
	SkeletalMeshEditorModule.OnRegisterLayoutExtensions().RemoveAll(this);
}

void USkeletalMeshEditorUISubsystem::RegisterLayoutExtensions(FLayoutExtender& Extender)
{
}

void FSkeletalMeshEditorModeUILayer::OnToolkitHostingStarted(const TSharedRef<IToolkit>& Toolkit)
{
	if (!Toolkit->IsAssetEditor())		// We only want to host Mode toolkits
	{
		FAssetEditorModeUILayer::OnToolkitHostingStarted(Toolkit);
		HostedToolkit = Toolkit;
		Toolkit->SetModeUILayer(SharedThis(this));
		Toolkit->RegisterTabSpawners(ToolkitHost->GetTabManager().ToSharedRef());
		RegisterModeTabSpawners();
		OnToolkitHostReadyForUI.ExecuteIfBound();
	}
}

void FSkeletalMeshEditorModeUILayer::OnToolkitHostingFinished(const TSharedRef<IToolkit>& Toolkit)
{
	if (HostedToolkit.IsValid() && HostedToolkit.Pin() == Toolkit)	// don't execute OnToolkitHostShutdownUI if the input Toolkit isn't the one we're hosting
	{
		FAssetEditorModeUILayer::OnToolkitHostingFinished(Toolkit);
	}	
}

TSharedPtr<FWorkspaceItem> FSkeletalMeshEditorModeUILayer::GetModeMenuCategory() const
{
	return SkeletalMeshEditorMenuCategory;
}

void FSkeletalMeshEditorModeUILayer::SetModeMenuCategory(TSharedPtr<FWorkspaceItem> InMenuCategory)
{
	SkeletalMeshEditorMenuCategory = InMenuCategory;
}
