// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Toolkits/AssetEditorModeUILayer.h"

#include "SkeletalMeshEditorModeUILayer.generated.h"

#define UE_API SKELETALMESHEDITOR_API

/** Interchange layer to manage built in tab locations within the editor's layout. **/
UCLASS(MinimalAPI)
class USkeletalMeshEditorUISubsystem : public UAssetEditorUISubsystem
{
	GENERATED_BODY()
public:

	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void Deinitialize() override;
	UE_API virtual void RegisterLayoutExtensions(FLayoutExtender& Extender) override;

};

/** Handles the hosting of additional toolkits, such as the mode toolkit, within the Skeletal Mesh Editor 's toolkit. **/
class FSkeletalMeshEditorModeUILayer : public FAssetEditorModeUILayer
{
public:
	FSkeletalMeshEditorModeUILayer(const IToolkitHost* InToolkitHost) : FAssetEditorModeUILayer(InToolkitHost) {}
	virtual void OnToolkitHostingStarted(const TSharedRef<IToolkit>& Toolkit) override;
	virtual void OnToolkitHostingFinished(const TSharedRef<IToolkit>& Toolkit) override;

	virtual TSharedPtr<FWorkspaceItem> GetModeMenuCategory() const override;

	void SetModeMenuCategory(TSharedPtr<FWorkspaceItem> InMenuCategory);
protected:
	
	TSharedPtr<FWorkspaceItem> SkeletalMeshEditorMenuCategory;
};



#undef UE_API
