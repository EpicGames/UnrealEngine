// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "ISkeletalMeshEditor.h"
#include "ISkeletalMeshEditorModule.h"
#include "SkeletalMeshEditor.h"

class FSkeletalMeshEditorModule : public ISkeletalMeshEditorModule
{
public:
	/** Constructor, set up console commands and variables **/
	FSkeletalMeshEditorModule()
	{
	}

	/** Called right after the module DLL has been loaded and the module object has been created */
	virtual void StartupModule() override
	{
		MenuExtensibilityManager = MakeShareable(new FExtensibilityManager);
		ToolBarExtensibilityManager = MakeShareable(new FExtensibilityManager);
	}

	/** Called before the module is unloaded, right before the module object is destroyed. */
	virtual void ShutdownModule() override
	{
		MenuExtensibilityManager.Reset();
		ToolBarExtensibilityManager.Reset();
	}

	virtual TSharedRef<ISkeletalMeshEditor> CreateSkeletalMeshEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, USkeletalMesh* InSkeletalMesh) override
	{
		TSharedRef<FSkeletalMeshEditor> SkeletalMeshEditor(new FSkeletalMeshEditor());
		SkeletalMeshEditor->InitSkeletalMeshEditor(Mode, InitToolkitHost, InSkeletalMesh);
		return SkeletalMeshEditor;
	}

	virtual TArray<FOnSkeletalMeshEditorInitialized>& GetPostEditorInitDelegates() override { return SkeletalMeshEditorInitialized; }
	virtual TArray<FSkeletalMeshEditorToolbarExtender>& GetAllSkeletalMeshEditorToolbarExtenders() override { return SkeletalMeshEditorToolbarExtenders; }

	/** Gets the extensibility managers for outside entities to extend this editor's menus and toolbars */
	virtual TSharedPtr<FExtensibilityManager> GetMenuExtensibilityManager() override { return MenuExtensibilityManager; }
	virtual TSharedPtr<FExtensibilityManager> GetToolBarExtensibilityManager() override { return ToolBarExtensibilityManager; }

	virtual FOnRegisterLayoutExtensions& OnRegisterLayoutExtensions() override { return RegisterLayoutExtensions; }
	
private:
	TSharedPtr<FExtensibilityManager> MenuExtensibilityManager;
	TSharedPtr<FExtensibilityManager> ToolBarExtensibilityManager;

	TArray<FOnSkeletalMeshEditorInitialized> SkeletalMeshEditorInitialized;
	TArray<FSkeletalMeshEditorToolbarExtender> SkeletalMeshEditorToolbarExtenders;
	
	FOnRegisterLayoutExtensions	RegisterLayoutExtensions;
};

IMPLEMENT_MODULE(FSkeletalMeshEditorModule, SkeletalMeshEditor);
