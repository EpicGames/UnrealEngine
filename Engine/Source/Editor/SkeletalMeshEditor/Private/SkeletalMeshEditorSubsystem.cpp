// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletalMeshEditorSubsystem.h"
#include "Subsystems/UnrealEditorSubsystem.h"
#include "Animation/Skeleton.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Components/SkeletalMeshComponent.h"
#include "Editor.h"
#include "EditorFramework/AssetImportData.h"
#include "Engine/EngineTypes.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkeletalMeshLODSettings.h"
#include "Engine/SkeletalMeshSocket.h"
#include "Engine/SkinnedAssetCommon.h"
#include "FbxMeshUtils.h"
#include "LODUtilities.h"
#include "Misc/ScopedSlowTask.h"
#include "ObjectTools.h"
#include "PhysicsAssetUtils.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/PhysicsConstraintTemplate.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "Rendering/SkeletalMeshLODModel.h"
#include "Rendering/SkeletalMeshLODRenderData.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "ScopedTransaction.h"
#include "EditorScriptingHelpers.h"
#include "SkeletalMeshTypes.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Subsystems/ImportSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SkeletalMeshEditorSubsystem)

#define LOCTEXT_NAMESPACE "SkeletalMeshEditorSubsystem"

DEFINE_LOG_CATEGORY(LogSkeletalMeshEditorSubsystem);

USkeletalMeshEditorSubsystem::USkeletalMeshEditorSubsystem()
	: UEditorSubsystem()
{

}

bool USkeletalMeshEditorSubsystem::RegenerateLOD(USkeletalMesh* SkeletalMesh, int32 NewLODCount /*= 0*/, bool bRegenerateEvenIfImported /*= false*/, bool bGenerateBaseLOD /*= false*/)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return false;
	}

	if (SkeletalMesh == nullptr)
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "RegenerateLOD: The SkeletalMesh is null.");
		return false;
	}

	return FLODUtilities::RegenerateLOD(SkeletalMesh, GetTargetPlatformManagerRef().GetRunningTargetPlatform(), NewLODCount, bRegenerateEvenIfImported, bGenerateBaseLOD);
}


int32 USkeletalMeshEditorSubsystem::GetNumVerts(USkeletalMesh* SkeletalMesh, int32 LODIndex)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return 0;
	}

	if (SkeletalMesh == nullptr)
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "GetNumVerts: The SkeletalMesh is null.");
		return 0;
	}

	if (SkeletalMesh->GetResourceForRendering() && SkeletalMesh->GetResourceForRendering()->LODRenderData.Num() > 0)
	{
		TIndirectArray<FSkeletalMeshLODRenderData>& LodRenderData = SkeletalMesh->GetResourceForRendering()->LODRenderData;
		if (LodRenderData.IsValidIndex(LODIndex))
		{
			return LodRenderData[LODIndex].GetNumVertices();
		}
	}

	return 0;
}

int32 USkeletalMeshEditorSubsystem::GetNumSections( USkeletalMesh* SkeletalMesh, int32 LODIndex )
{
	TGuardValue<bool> UnattendedScriptGuard( GIsRunningUnattendedScript, true );

	if ( !EditorScriptingHelpers::CheckIfInEditorAndPIE() )
	{
		return INDEX_NONE;
	}

	if ( SkeletalMesh == nullptr )
	{
		UE_LOGF( LogSkeletalMeshEditorSubsystem, Error, "GetNumSections: The SkeletalMesh is null." );
		return INDEX_NONE;
	}

	if ( SkeletalMesh->GetResourceForRendering() && SkeletalMesh->GetResourceForRendering()->LODRenderData.Num() > 0 )
	{
		TIndirectArray<FSkeletalMeshLODRenderData>& LodRenderData = SkeletalMesh->GetResourceForRendering()->LODRenderData;
		if ( LodRenderData.IsValidIndex( LODIndex ) )
		{
			return LodRenderData[ LODIndex ].RenderSections.Num();
		}
	}

	return INDEX_NONE;
}

bool USkeletalMeshEditorSubsystem::GetSectionRecomputeTangent(const USkeletalMesh* SkeletalMesh, const int32 LODIndex, const int32 SectionIndex, bool& bOutRecomputeTangent)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return false;
	}

	if (SkeletalMesh == nullptr)
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "GetSectionRecomputeTangent: The SkeletalMesh is null.");
		return false;
	}

	if (!SkeletalMesh->GetResourceForRendering() || !(SkeletalMesh->GetResourceForRendering()->LODRenderData.Num() > 0))
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "GetSectionRecomputeTangent: The render data is null.");
		return false;
	}

	TIndirectArray<FSkeletalMeshLODRenderData>& LodRenderData = SkeletalMesh->GetResourceForRendering()->LODRenderData;
	if (!LodRenderData.IsValidIndex(LODIndex))
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "GetSectionRecomputeTangent: The LOD index is invalid.");
		return false;
	}
	if (!LodRenderData[LODIndex].RenderSections.IsValidIndex(SectionIndex))
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "GetSectionRecomputeTangent: The section index is invalid.");
		return false;
	}

	bOutRecomputeTangent = LodRenderData[LODIndex].RenderSections[SectionIndex].bRecomputeTangent;
	return true;
}

bool USkeletalMeshEditorSubsystem::SetSectionRecomputeTangent(USkeletalMesh* SkeletalMesh, const int32 LODIndex, const int32 SectionIndex, const bool bRecomputeTangent)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return false;
	}

	if (SkeletalMesh == nullptr)
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "SetSectionRecomputeTangent: The SkeletalMesh is null.");
		return false;
	}

	if (!SkeletalMesh->GetImportedModel())
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "SetSectionRecomputeTangent: The imported model is null.");
		return false;
	}

	if (!SkeletalMesh->GetImportedModel()->LODModels.IsValidIndex(LODIndex))
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "SetSectionRecomputeTangent: The LOD index is invalid.");
		return false;
	}
	FSkeletalMeshLODModel& LODModel = SkeletalMesh->GetImportedModel()->LODModels[LODIndex];
	if (!LODModel.Sections.IsValidIndex(SectionIndex))
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "SetSectionRecomputeTangent: The section index is invalid.");
		return false;
	}


	//Make the change as a transaction and make sure the render data is rebuilt properly
	{
		FScopedTransaction Transaction(LOCTEXT("SetSectionRecomputeTangentTransactionName", "Set section recompute tangent"));
		SkeletalMesh->Modify();
		const int32 OriginalSectionIndex = LODModel.Sections[SectionIndex].OriginalDataSectionIndex;
		FSkelMeshSourceSectionUserData& SectionUserData = LODModel.UserSectionsData.FindOrAdd(OriginalSectionIndex);
		SectionUserData.bRecomputeTangent = bRecomputeTangent;
	}
	//Rebuild the render data
	SkeletalMesh->PostEditChange();
	//This will make the sk build synchronous
	bool bNewValue = false;
	GetSectionRecomputeTangent(SkeletalMesh, LODIndex, SectionIndex, bNewValue);
	ensure(bNewValue == bRecomputeTangent);
	return true;
}

bool USkeletalMeshEditorSubsystem::GetSectionRecomputeTangentsVertexMaskChannel(const USkeletalMesh* SkeletalMesh, const int32 LODIndex, const int32 SectionIndex, uint8& OutRecomputeTangentsVertexMaskChannel)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return false;
	}

	if (SkeletalMesh == nullptr)
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "GetSectionRecomputeTangentsVertexMaskChannel: The SkeletalMesh is null.");
		return false;
	}

	if (!SkeletalMesh->GetResourceForRendering() || !(SkeletalMesh->GetResourceForRendering()->LODRenderData.Num() > 0))
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "GetSectionRecomputeTangentsVertexMaskChannel: The render data is null.");
		return false;
	}

	TIndirectArray<FSkeletalMeshLODRenderData>& LodRenderData = SkeletalMesh->GetResourceForRendering()->LODRenderData;
	if (!LodRenderData.IsValidIndex(LODIndex))
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "GetSectionRecomputeTangentsVertexMaskChannel: The LOD index is invalid.");
		return false;
	}
	if (!LodRenderData[LODIndex].RenderSections.IsValidIndex(SectionIndex))
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "GetSectionRecomputeTangentsVertexMaskChannel: The section index is invalid.");
		return false;
	}

	OutRecomputeTangentsVertexMaskChannel = static_cast<uint8>(LodRenderData[LODIndex].RenderSections[SectionIndex].RecomputeTangentsVertexMaskChannel);
	return true;
}

bool USkeletalMeshEditorSubsystem::SetSectionRecomputeTangentsVertexMaskChannel(USkeletalMesh* SkeletalMesh, const int32 LODIndex, const int32 SectionIndex, const uint8 RecomputeTangentsVertexMaskChannel)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return false;
	}

	if (SkeletalMesh == nullptr)
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "SetSectionRecomputeTangentsVertexMaskChannel: The SkeletalMesh is null.");
		return false;
	}

	if (!SkeletalMesh->GetImportedModel())
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "SetSectionRecomputeTangentsVertexMaskChannel: The imported model is null.");
		return false;
	}

	if (!SkeletalMesh->GetImportedModel()->LODModels.IsValidIndex(LODIndex))
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "SetSectionRecomputeTangentsVertexMaskChannel: The LOD index is invalid.");
		return false;
	}
	FSkeletalMeshLODModel& LODModel = SkeletalMesh->GetImportedModel()->LODModels[LODIndex];
	if (!LODModel.Sections.IsValidIndex(SectionIndex))
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "SetSectionRecomputeTangentsVertexMaskChannel: The section index is invalid.");
		return false;
	}

	//Make the change as a transaction and make sure the render data is rebuilt properly
	{
		FScopedTransaction Transaction(LOCTEXT("SetSectionRecomputeTangentsVertexMaskChannelTransactionName", "Set section recompute tangents vertex mask channel"));
		SkeletalMesh->Modify();
		const int32 OriginalSectionIndex = LODModel.Sections[SectionIndex].OriginalDataSectionIndex;
		FSkelMeshSourceSectionUserData& SectionUserData = LODModel.UserSectionsData.FindOrAdd(OriginalSectionIndex);
		SectionUserData.RecomputeTangentsVertexMaskChannel = static_cast<ESkinVertexColorChannel>(RecomputeTangentsVertexMaskChannel);
	}
	
	SkeletalMesh->PostEditChange();
	//This will make the sk build synchronous
	uint8 NewValue;
	GetSectionRecomputeTangentsVertexMaskChannel(SkeletalMesh, LODIndex, SectionIndex, NewValue);
	ensure(NewValue == RecomputeTangentsVertexMaskChannel);
	
	return true;
}

bool USkeletalMeshEditorSubsystem::GetSectionCastShadow(const USkeletalMesh* SkeletalMesh, const int32 LODIndex, const int32 SectionIndex, bool& bOutCastShadow)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return false;
	}

	if (SkeletalMesh == nullptr)
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "GetSectionCastShadow: The SkeletalMesh is null.");
		return false;
	}

	if (!SkeletalMesh->GetResourceForRendering() || !(SkeletalMesh->GetResourceForRendering()->LODRenderData.Num() > 0))
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "GetSectionCastShadow: The render data is null.");
		return false;
	}

	TIndirectArray<FSkeletalMeshLODRenderData>& LodRenderData = SkeletalMesh->GetResourceForRendering()->LODRenderData;
	if (!LodRenderData.IsValidIndex(LODIndex))
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "GetSectionCastShadow: The LOD index is invalid.");
		return false;
	}
	if (!LodRenderData[LODIndex].RenderSections.IsValidIndex(SectionIndex))
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "GetSectionCastShadow: The section index is invalid.");
		return false;
	}

	bOutCastShadow = LodRenderData[LODIndex].RenderSections[SectionIndex].bCastShadow;
	return true;
}

bool USkeletalMeshEditorSubsystem::SetSectionCastShadow(USkeletalMesh* SkeletalMesh, const int32 LODIndex, const int32 SectionIndex, const bool bCastShadow)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return false;
	}

	if (SkeletalMesh == nullptr)
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "SetSectionCastShadow: The SkeletalMesh is null.");
		return false;
	}

	if (!SkeletalMesh->GetImportedModel())
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "SetSectionCastShadow: The imported model is null.");
		return false;
	}

	if (!SkeletalMesh->GetImportedModel()->LODModels.IsValidIndex(LODIndex))
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "SetSectionCastShadow: The LOD index is invalid.");
		return false;
	}
	FSkeletalMeshLODModel& LODModel = SkeletalMesh->GetImportedModel()->LODModels[LODIndex];
	if (!LODModel.Sections.IsValidIndex(SectionIndex))
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "SetSectionCastShadow: The section index is invalid.");
		return false;
	}

	//Make the change as a transaction and make sure the render data is rebuilt properly
	{
		FScopedTransaction Transaction(LOCTEXT("SetSectionCastShadowTransactionName", "Set section cast shadow"));
		SkeletalMesh->Modify();
		const int32 OriginalSectionIndex = LODModel.Sections[SectionIndex].OriginalDataSectionIndex;
		FSkelMeshSourceSectionUserData& SectionUserData = LODModel.UserSectionsData.FindOrAdd(OriginalSectionIndex);
		SectionUserData.bCastShadow = bCastShadow;
	}
	//Rebuild the render data
	SkeletalMesh->PostEditChange();
	//This will make the sk build synchronous
	bool bNewValue = false;
	GetSectionCastShadow(SkeletalMesh, LODIndex, SectionIndex, bNewValue);
	ensure(bNewValue == bCastShadow);

	return true;
}

bool USkeletalMeshEditorSubsystem::GetSectionVisibleInRayTracing(const USkeletalMesh* SkeletalMesh, const int32 LODIndex, const int32 SectionIndex, bool& bOutVisibleInRayTracing)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return false;
	}

	if (SkeletalMesh == nullptr)
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "GetSectionVisibleInRayTracing: The SkeletalMesh is null.");
		return false;
	}

	if (!SkeletalMesh->GetResourceForRendering() || !(SkeletalMesh->GetResourceForRendering()->LODRenderData.Num() > 0))
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "GetSectionVisibleInRayTracing: The render data is null.");
		return false;
	}

	TIndirectArray<FSkeletalMeshLODRenderData>& LodRenderData = SkeletalMesh->GetResourceForRendering()->LODRenderData;
	if (!LodRenderData.IsValidIndex(LODIndex))
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "GetSectionVisibleInRayTracing: The LOD index is invalid.");
		return false;
	}
	if (!LodRenderData[LODIndex].RenderSections.IsValidIndex(SectionIndex))
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "GetSectionVisibleInRayTracing: The section index is invalid.");
		return false;
	}

	bOutVisibleInRayTracing = LodRenderData[LODIndex].RenderSections[SectionIndex].bVisibleInRayTracing;
	return true;
}

bool USkeletalMeshEditorSubsystem::SetSectionVisibleInRayTracing(USkeletalMesh* SkeletalMesh, const int32 LODIndex, const int32 SectionIndex, const bool bVisibleInRayTracing)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return false;
	}

	if (SkeletalMesh == nullptr)
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "SetSectionVisibleInRayTracing: The SkeletalMesh is null.");
		return false;
	}

	if (!SkeletalMesh->GetImportedModel())
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "SetSectionVisibleInRayTracing: The imported model is null.");
		return false;
	}

	if (!SkeletalMesh->GetImportedModel()->LODModels.IsValidIndex(LODIndex))
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "SetSectionVisibleInRayTracing: The LOD index is invalid.");
		return false;
	}
	FSkeletalMeshLODModel& LODModel = SkeletalMesh->GetImportedModel()->LODModels[LODIndex];
	if (!LODModel.Sections.IsValidIndex(SectionIndex))
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "SetSectionVisibleInRayTracing: The section index is invalid.");
		return false;
	}

	//Make the change as a transaction and make sure the render data is rebuilt properly
	{
		FScopedTransaction Transaction(LOCTEXT("SetSectionVisibleInRayTracingTransactionName", "Set section visible in ray tracing"));
		SkeletalMesh->Modify();
		const int32 OriginalSectionIndex = LODModel.Sections[SectionIndex].OriginalDataSectionIndex;
		FSkelMeshSourceSectionUserData& SectionUserData = LODModel.UserSectionsData.FindOrAdd(OriginalSectionIndex);
		SectionUserData.bVisibleInRayTracing = bVisibleInRayTracing;
	}

	SkeletalMesh->PostEditChange();
	//This will make the sk build synchronous
	bool bNewValue = false;
	GetSectionVisibleInRayTracing(SkeletalMesh, LODIndex, SectionIndex, bNewValue);
	ensure(bNewValue == bVisibleInRayTracing);

	return true;
}

UMaterialInterface* USkeletalMeshEditorSubsystem::GetMaterialSlotOverlayMaterial(const USkeletalMesh* SkeletalMesh, const int32 SlotIndex)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return nullptr;
	}

	if (SkeletalMesh == nullptr)
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "GetMaterialSlotOverlayMaterial: The SkeletalMesh is null.");
		return nullptr;
	}

	const TArray<FSkeletalMaterial>& SkeletalMaterials = SkeletalMesh->GetMaterials();
	if (!SkeletalMaterials.IsValidIndex(SlotIndex))
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "GetMaterialSlotOverlayMaterial: Invalid skeletal mesh material slot index.");
		return nullptr;
	}
	return SkeletalMaterials[SlotIndex].OverlayMaterialInterface;
}

bool USkeletalMeshEditorSubsystem::SetMaterialSlotOverlayMaterial(USkeletalMesh* SkeletalMesh, const int32 SlotIndex, UMaterialInterface* NewSectionOverlayMaterial)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return false;
	}

	if (SkeletalMesh == nullptr)
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "GetMaterialSlotOverlayMaterial: The SkeletalMesh is null.");
		return false;
	}

	TArray<FSkeletalMaterial>& SkeletalMaterials = SkeletalMesh->GetMaterials();
	if (!SkeletalMaterials.IsValidIndex(SlotIndex))
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "GetMaterialSlotOverlayMaterial: Invalid skeletal mesh material slot index.");
		return false;
	}

	//Make the change as a transaction and make sure the render data is rebuilt properly
	{
		FScopedTransaction Transaction(LOCTEXT("SetSectionOverlayMaterialTransactionName", "Set skeletal mesh section overlay material"));
		SkeletalMesh->Modify();
		SkeletalMaterials[SlotIndex] = NewSectionOverlayMaterial;
	}

	SkeletalMesh->PostEditChange();
	//This will make the sk build synchronous
	UMaterialInterface* MaterialValue = GetMaterialSlotOverlayMaterial(SkeletalMesh, SlotIndex);
	ensure(MaterialValue == NewSectionOverlayMaterial);
	return true;
}

UMaterialInterface* USkeletalMeshEditorSubsystem::GetSkeletalMeshOverlayMaterial(const USkeletalMesh* SkeletalMesh)
{
	if (!SkeletalMesh)
	{
		return nullptr;
	}
	return SkeletalMesh->GetOverlayMaterial();
}

bool USkeletalMeshEditorSubsystem::SetSkeletalMeshOverlayMaterial(USkeletalMesh* SkeletalMesh, UMaterialInterface* NewOverlayMaterial)
{
	if (!SkeletalMesh)
	{
		return false;
	}

	//Transaction scope
	{
		FScopedTransaction Transaction(LOCTEXT("SetSkeletalMeshOverlayMaterialTransactionName", "Set skeletal mesh overlay material"));
		SkeletalMesh->Modify();
		SkeletalMesh->SetOverlayMaterial(NewOverlayMaterial);
	}
	SkeletalMesh->PostEditChange();

	UMaterialInterface* MaterialValue = GetSkeletalMeshOverlayMaterial(SkeletalMesh);
	ensure(MaterialValue == NewOverlayMaterial);
	return true;
}

int32 USkeletalMeshEditorSubsystem::GetLODMaterialSlot( USkeletalMesh* SkeletalMesh, int32 LODIndex, int32 SectionIndex )
{
	TGuardValue<bool> UnattendedScriptGuard( GIsRunningUnattendedScript, true );

	if ( !EditorScriptingHelpers::CheckIfInEditorAndPIE() )
	{
		return INDEX_NONE;
	}

	if ( SkeletalMesh == nullptr )
	{
		UE_LOGF( LogSkeletalMeshEditorSubsystem, Error, "GetLODMaterialSlot: The SkeletalMesh is null." );
		return INDEX_NONE;
	}

	if ( SkeletalMesh->GetResourceForRendering() && SkeletalMesh->GetResourceForRendering()->LODRenderData.Num() > 0 )
	{
		TIndirectArray<FSkeletalMeshLODRenderData>& LodRenderData = SkeletalMesh->GetResourceForRendering()->LODRenderData;
		if ( LodRenderData.IsValidIndex( LODIndex ) )
		{
			const TArray<FSkelMeshRenderSection>& Sections = LodRenderData[ LODIndex ].RenderSections;
			if ( Sections.IsValidIndex( SectionIndex ) )
			{
				int32 MaterialIndex = Sections[ SectionIndex ].MaterialIndex;

				const FSkeletalMeshLODInfo* LODInfo = SkeletalMesh->GetLODInfo( LODIndex );
				if ( LODInfo && LODInfo->LODMaterialMap.IsValidIndex( SectionIndex ) )
				{
					// If we have an optional LODMaterialMap, we need to reroute the material index through it
					MaterialIndex = LODInfo->LODMaterialMap[ SectionIndex ];
				}

				return MaterialIndex;
			}
			else
			{
				UE_LOGF( LogSkeletalMeshEditorSubsystem, Error, "GetLODMaterialSlot: Invalid SectionIndex." );
			}
		}
		else
		{
			UE_LOGF( LogSkeletalMeshEditorSubsystem, Error, "GetLODMaterialSlot: Invalid LODIndex." );
		}
	}

	return INDEX_NONE;
}

bool USkeletalMeshEditorSubsystem::RenameSocket(USkeletalMesh* SkeletalMesh, FName OldName, FName NewName)
{
	if (SkeletalMesh == nullptr)
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "RenameSocket: The SkeletalMesh is null.");
		return false;
	}

	if (SkeletalMesh->GetSkeleton() == nullptr)
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "RenameSocket: The SkeletalMesh's Skeleton is null.");
		return false;
	}

	if (NewName == NAME_None)
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "RenameSocket: The NewName is None.");
		return false;
	}

	if (OldName == NewName)
	{
		return false;
	}

	USkeletalMeshSocket* MeshSocket = SkeletalMesh->FindSocket(OldName);
	if (MeshSocket == nullptr)
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "RenameSocket: The socket named '%ls' does not exist on the SkeletalMesh.", *OldName.ToString());
		return false;
	}

	USkeletalMeshSocket* SkeletonSocket = SkeletalMesh->GetSkeleton()->FindSocket(OldName);
	// If they're one and the same socket, it's because the USkeletalMesh::FindSocket falls back on searching the skeleton if it can't find the
	// socket on the mesh itself. 
	if (SkeletonSocket == MeshSocket)
	{
		SkeletonSocket = nullptr;
	}

	FScopedTransaction Transaction(LOCTEXT("RenameSocket", "Rename Socket"));
	MeshSocket->SetFlags(RF_Transactional);
	MeshSocket->Modify();
	MeshSocket->SocketName = NewName;

	if (SkeletonSocket)
	{
		SkeletonSocket->SetFlags(RF_Transactional);
		SkeletonSocket->Modify();
		SkeletonSocket->SocketName = NewName;
	}

	FPreviewAssetAttachContainer& PreviewAssetAttachContainer = SkeletalMesh->GetPreviewAttachedAssetContainer();
	bool bMeshModified = false;
	for (int AttachedObjectIndex = 0; AttachedObjectIndex < PreviewAssetAttachContainer.Num(); ++AttachedObjectIndex)
	{
		FPreviewAttachedObjectPair& Pair = PreviewAssetAttachContainer[AttachedObjectIndex];
		if (Pair.AttachedTo == OldName)
		{
			// Only modify the mesh if we actually intend to change something. Avoids dirtying
			// meshes when we don't actually update any data on them. (such as adding a new socket)
			if (!bMeshModified)
			{
				SkeletalMesh->Modify();
				bMeshModified = true;
			}
			Pair.AttachedTo = NewName;
		}
	}

	bool bSkeletonModified = false;
	for (int AttachedObjectIndex = 0; AttachedObjectIndex < SkeletalMesh->GetSkeleton()->PreviewAttachedAssetContainer.Num(); ++AttachedObjectIndex)
	{
		FPreviewAttachedObjectPair& Pair = SkeletalMesh->GetSkeleton()->PreviewAttachedAssetContainer[AttachedObjectIndex];
		if (Pair.AttachedTo == OldName)
		{
			// Only modify the skeleton if we actually intend to change something.
			if (!bSkeletonModified)
			{
				SkeletalMesh->GetSkeleton()->Modify();
				bSkeletonModified = true;
			}
			Pair.AttachedTo = NewName;
		}
	}

	return true;
}

bool USkeletalMeshEditorSubsystem::RemoveSocket(USkeletalMesh* SkeletalMesh, FName SocketName)
{
	if (SkeletalMesh == nullptr)
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "RemoveSocket: The SkeletalMesh is null.");
		return false;
	}

	if (SkeletalMesh->GetSkeleton() == nullptr)
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "RemoveSocket: The SkeletalMesh's Skeleton is null.");
		return false;
	}

	if (SocketName == NAME_None)
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "RemoveSocket: The SocketName is None.");
		return false;
	}

	USkeletalMeshSocket* FoundSocket = SkeletalMesh->FindSocket(SocketName);
	if (FoundSocket == nullptr)
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "RemoveSocket: The socket named '%ls' does not exist on the SkeletalMesh.", *SocketName.ToString());
		return false;
	}

	USkeletalMeshSocket* SkeletonSocket = SkeletalMesh->GetSkeleton()->FindSocket(SocketName);
	// If they're the same pointer, FindSocket fell back to the skeleton — there is no mesh-specific socket.
	if (SkeletonSocket == FoundSocket)
	{
		SkeletonSocket = nullptr;
	}

	FScopedTransaction Transaction(LOCTEXT("RemoveSocket", "Remove Socket"));

	TArray<TObjectPtr<USkeletalMeshSocket>>& MeshSocketList = SkeletalMesh->GetMeshOnlySocketList();
	if (MeshSocketList.Contains(FoundSocket))
	{
		FoundSocket->SetFlags(RF_Transactional);
		FoundSocket->Modify();
		SkeletalMesh->Modify();
		MeshSocketList.Remove(FoundSocket);
	}
	else
	{
		// Socket exists only on the skeleton (FindSocket fell through to it).
		FoundSocket->SetFlags(RF_Transactional);
		FoundSocket->Modify();
		SkeletalMesh->GetSkeleton()->Modify();
		SkeletalMesh->GetSkeleton()->Sockets.Remove(FoundSocket);
	}

	if (SkeletonSocket != nullptr)
	{
		SkeletonSocket->SetFlags(RF_Transactional);
		SkeletonSocket->Modify();
		SkeletalMesh->GetSkeleton()->Modify();
		SkeletalMesh->GetSkeleton()->Sockets.Remove(SkeletonSocket);
	}

	// Clean up any preview-attached assets referencing the removed socket on the mesh.
	bool bMeshModified = false;
	FPreviewAssetAttachContainer& MeshPreviewContainer = SkeletalMesh->GetPreviewAttachedAssetContainer();
	for (int32 Index = MeshPreviewContainer.Num() - 1; Index >= 0; --Index)
	{
		if (MeshPreviewContainer[Index].AttachedTo == SocketName)
		{
			if (!bMeshModified)
			{
				SkeletalMesh->Modify();
				bMeshModified = true;
			}
			MeshPreviewContainer.RemoveAtSwap(Index);
		}
	}

	// Clean up any preview-attached assets referencing the removed socket on the skeleton.
	bool bSkeletonModified = false;
	FPreviewAssetAttachContainer& SkeletonPreviewContainer = SkeletalMesh->GetSkeleton()->PreviewAttachedAssetContainer;
	for (int32 Index = SkeletonPreviewContainer.Num() - 1; Index >= 0; --Index)
	{
		if (SkeletonPreviewContainer[Index].AttachedTo == SocketName)
		{
			if (!bSkeletonModified)
			{
				SkeletalMesh->GetSkeleton()->Modify();
				bSkeletonModified = true;
			}
			SkeletonPreviewContainer.RemoveAtSwap(Index);
		}
	}

	return true;
}

FName USkeletalMeshEditorSubsystem::GetBoneParent(const USkeletalMesh* SkeletalMesh, FName BoneName)
{
	if (!SkeletalMesh)
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "GetBoneParent: The SkeletalMesh is null.");
		return NAME_None;
	}

	const FReferenceSkeleton& RefSkel = SkeletalMesh->GetRefSkeleton();
	const int32 BoneIndex = RefSkel.FindBoneIndex(BoneName);
	if (BoneIndex == INDEX_NONE)
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "GetBoneParent: Bone '%ls' not found.", *BoneName.ToString());
		return NAME_None;
	}

	const int32 ParentIndex = RefSkel.GetParentIndex(BoneIndex);
	if (ParentIndex == INDEX_NONE)
	{
		return NAME_None;
	}

	return RefSkel.GetBoneName(ParentIndex);
}

TArray<FName> USkeletalMeshEditorSubsystem::GetBoneChildren(const USkeletalMesh* SkeletalMesh, FName BoneName)
{
	TArray<FName> Children;

	if (!SkeletalMesh)
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "GetBoneChildren: The SkeletalMesh is null.");
		return Children;
	}

	const FReferenceSkeleton& RefSkel = SkeletalMesh->GetRefSkeleton();
	const int32 BoneIndex = RefSkel.FindBoneIndex(BoneName);
	if (BoneIndex == INDEX_NONE)
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "GetBoneChildren: Bone '%ls' not found.", *BoneName.ToString());
		return Children;
	}

	const int32 NumBones = RefSkel.GetNum();
	for (int32 i = 0; i < NumBones; ++i)
	{
		if (RefSkel.GetParentIndex(i) == BoneIndex)
		{
			Children.Add(RefSkel.GetBoneName(i));
		}
	}

	return Children;
}

int32 USkeletalMeshEditorSubsystem::GetLODCount(USkeletalMesh* SkeletalMesh)
{
	if (SkeletalMesh == nullptr)
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "SkeletalMesh GetLODCount: The SkeletalMesh is null.");
		return INDEX_NONE;
	}

	return SkeletalMesh->GetLODNum();
}

int32 USkeletalMeshEditorSubsystem::ImportLOD(USkeletalMesh* BaseMesh, const int32 LODIndex, const FString& SourceFilename)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "SkeletalMesh ImportLOD: Cannot import or re-import when editor PIE is active.");
		return INDEX_NONE;
	}

	if (BaseMesh == nullptr)
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "SkeletalMesh ImportLOD: The SkeletalMesh is null.");
		return INDEX_NONE;
	}

	// Make sure the LODIndex we want to add the LOD is valid
	if (BaseMesh->GetLODNum() < LODIndex)
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "SkeletalMesh ImportLOD: Invalid LODIndex, the LOD index cannot be greater the the number of LOD, skeletal mesh cannot have hole in the LOD array.");
		return INDEX_NONE;
	}

	FString ResolveFilename = SourceFilename;
	const bool bSourceFileExists = FPaths::FileExists(ResolveFilename);
	if (!bSourceFileExists)
	{
		if (BaseMesh->IsValidLODIndex(LODIndex))
		{
			ResolveFilename = BaseMesh->GetLODInfo(LODIndex)->SourceImportFilename.IsEmpty() ?
				BaseMesh->GetLODInfo(LODIndex)->SourceImportFilename :
				UAssetImportData::ResolveImportFilename(BaseMesh->GetLODInfo(LODIndex)->SourceImportFilename, nullptr);
		}
	}

	if (!FPaths::FileExists(ResolveFilename))
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "SkeletalMesh ImportLOD: Invalid source filename.");
		return INDEX_NONE;
	}

	//We want to remove the Reduction this LOD if the following conditions are met
	if (BaseMesh->IsValidLODIndex(LODIndex) //Only test valid LOD, this function can add LODs
		&& BaseMesh->IsReductionActive(LODIndex) //We remove reduction settings only if they are active
		&& BaseMesh->GetLODInfo(LODIndex) //this test is redundant (IsReductionActive test this), but to avoid static analysis
		&& BaseMesh->GetLODInfo(LODIndex)->ReductionSettings.BaseLOD < LODIndex //We do not remove the reduction if the reduction is base on this LOD imported data
		&& (!BaseMesh->GetLODSettings() || BaseMesh->GetLODSettings()->GetNumberOfSettings() < LODIndex)) //We do not remove the reduction if the skeletal mesh is using a LODSettings for this LOD
	{
		//Remove the reduction settings
		BaseMesh->GetLODInfo(LODIndex)->ReductionSettings.NumOfTrianglesPercentage = 1.0f;
		BaseMesh->GetLODInfo(LODIndex)->ReductionSettings.NumOfVertPercentage = 1.0f;
		BaseMesh->GetLODInfo(LODIndex)->ReductionSettings.MaxNumOfTrianglesPercentage = MAX_uint32;
		BaseMesh->GetLODInfo(LODIndex)->ReductionSettings.MaxNumOfVertsPercentage = MAX_uint32;
		BaseMesh->GetLODInfo(LODIndex)->ReductionSettings.TerminationCriterion = SMTC_NumOfTriangles;

		BaseMesh->GetLODInfo(LODIndex)->bHasBeenSimplified = false;
	}
	constexpr bool bAsyncFalse = false;
	if (!FbxMeshUtils::ImportSkeletalMeshLOD(BaseMesh, ResolveFilename, LODIndex, bAsyncFalse))
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "SkeletalMesh ImportLOD: Cannot import mesh LOD.");
		return INDEX_NONE;
	}

	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostLODImport(BaseMesh, LODIndex);

	return LODIndex;
}

bool USkeletalMeshEditorSubsystem::RemoveLODs(USkeletalMesh* BaseMesh, const TArray<int32> ToRemoveLODs)
{
	FSkeletalMeshUpdateContext UpdateContext;
	UpdateContext.SkeletalMesh = BaseMesh;
	int32 OriginalLODNumber = BaseMesh->GetLODNum();

	// Close the mesh editor to be sure the editor is showing the correct data after the LODs are removed.
	bool bMeshIsEdited = false;
	UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
	if (AssetEditorSubsystem->FindEditorForAsset(BaseMesh, false))
	{
		AssetEditorSubsystem->CloseAllEditorsForAsset(BaseMesh);
		bMeshIsEdited = true;
	}

	// Now iterate over all skeletal mesh components to add them to the UpdateContext
	for (TObjectIterator<USkeletalMeshComponent> It; It; ++It)
	{
		USkeletalMeshComponent* SkelComp = *It;
		if (SkelComp->GetSkeletalMeshAsset() == BaseMesh)
		{
			UpdateContext.AssociatedComponents.Add(SkelComp);
		}
	}

	FLODUtilities::RemoveLODs(UpdateContext, ToRemoveLODs);

	if (bMeshIsEdited)
	{
		AssetEditorSubsystem->OpenEditorForAsset(BaseMesh);
	}

	int32 FinalLODNumber = BaseMesh->GetLODNum();
	return (OriginalLODNumber - FinalLODNumber == ToRemoveLODs.Num());
}

bool USkeletalMeshEditorSubsystem::StripLODGeometry(USkeletalMesh* SkeletalMesh, const int32 LODIndex, UTexture2D* TextureMask, const float Threshold)
{
	return FLODUtilities::StripLODGeometry(SkeletalMesh, LODIndex, TextureMask, Threshold);
}

bool USkeletalMeshEditorSubsystem::ReimportAllCustomLODs(USkeletalMesh* SkeletalMesh)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "SkeletalMesh ReimportAllCustomLODs: Cannot import or re-import when editor PIE is active.");
		return false;
	}

	if (SkeletalMesh == nullptr)
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "SkeletalMesh ReimportAllCustomLODs: The SkeletalMesh is null.");
		return false;
	}

	bool bResult = true;
	int32 LODNumber = SkeletalMesh->GetLODNum();
	//Iterate the static mesh LODs, start at index 1
	for (int32 LODIndex = 1; LODIndex < LODNumber; ++LODIndex)
	{
		//Do not reimport LOD that was re-import with the base mesh
		if (SkeletalMesh->GetLODInfo(LODIndex)->bImportWithBaseMesh)
		{
			continue;
		}
		if (SkeletalMesh->GetLODInfo(LODIndex)->bHasBeenSimplified)
		{
			continue;
		}

		if (ImportLOD(SkeletalMesh, LODIndex, SkeletalMesh->GetLODInfo(LODIndex)->SourceImportFilename) != LODIndex)
		{
			UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "SkeletalMesh ReimportAllCustomLODs: Cannot re-import LOD %d.", LODIndex);
			bResult = false;
		}
	}
	return bResult;
}

void USkeletalMeshEditorSubsystem::GetLodBuildSettings(const USkeletalMesh* SkeletalMesh, const int32 LodIndex, FSkeletalMeshBuildSettings& OutBuildOptions)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return;
	}

	if (SkeletalMesh == nullptr)
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "GetLodReductionSettings: The SkeletalMesh is null.");
		return;
	}

	// If LOD 0 does not exist, warn and return
	if (LodIndex < 0 || SkeletalMesh->GetLODNum() <= LodIndex)
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "GetLodReductionSettings: Invalid LOD index.");
		return;
	}

	const FSkeletalMeshLODInfo* LODInfo = SkeletalMesh->GetLODInfo(LodIndex);
	//LodIndex validity was verify before
	check(LODInfo);

	// Copy over the reduction settings
	OutBuildOptions = LODInfo->BuildSettings;
}

void USkeletalMeshEditorSubsystem::SetLodBuildSettings(USkeletalMesh* SkeletalMesh, const int32 LodIndex, const FSkeletalMeshBuildSettings& BuildOptions)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return;
	}

	if (SkeletalMesh == nullptr)
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "SetLodReductionSettings: The SkeletalMesh is null.");
		return;
	}

	// If LOD 0 does not exist, warn and return
	if (LodIndex < 0 || SkeletalMesh->GetLODNum() <= LodIndex)
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "GetLodReductionSettings: Invalid LOD index.");
		return;
	}

	FSkeletalMeshLODInfo* LODInfo = SkeletalMesh->GetLODInfo(LodIndex);
	//LodIndex validity was verify before
	check(LODInfo);

	// Close the mesh editor to prevent crashing. If changes are applied, reopen it after the mesh has been built.
	bool bSkeletalMeshIsEdited = false;
	UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
	if (AssetEditorSubsystem->FindEditorForAsset(SkeletalMesh, false))
	{
		AssetEditorSubsystem->CloseAllEditorsForAsset(SkeletalMesh);
		bSkeletalMeshIsEdited = true;
	}

	//Copy the reduction setting on the LODInfo
	{
		FScopedSkeletalMeshPostEditChange ScopedSkeletalMeshPostEditChange(SkeletalMesh);
		SkeletalMesh->Modify();

		// Copy over the reduction settings
		LODInfo->BuildSettings = BuildOptions;
	}
	// Reopen MeshEditor on this mesh if the MeshEditor was previously opened in it
	if (bSkeletalMeshIsEdited)
	{
		AssetEditorSubsystem->OpenEditorForAsset(SkeletalMesh);
	}
}

UPhysicsAsset* USkeletalMeshEditorSubsystem::CreatePhysicsAsset(USkeletalMesh* SkeletalMesh, bool bSetToMesh /*= true*/, int32 LodIndex /*= 0*/)
{
	if (!SkeletalMesh)
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "CreatePhysicsAsset failed: The SkeletalMesh is null.");
		return nullptr;
	}

	FString ObjectName = FString::Printf(TEXT("%s_PhysicsAsset"), *SkeletalMesh->GetName());
	FString PackageName = SkeletalMesh->GetOutermost()->GetName();

	FString ParentPath = FString::Printf(TEXT("%s/%s"), *FPackageName::GetLongPackagePath(*PackageName), *ObjectName);
	UObject* Package = CreatePackage(*ParentPath);

	// See if an object with this name exists
	UObject* Object = LoadObject<UObject>(Package, *ObjectName, nullptr, LOAD_NoWarn | LOAD_Quiet, nullptr);

	// If an object with same name but different class exists, fail and warn the user
	if ((Object != nullptr) && (Object->GetClass() != UPhysicsAsset::StaticClass()))
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "CreatePhysicsAsset failed: An object that is not a Physics Asset already exists with the name %ls.", *ObjectName);
		return nullptr;
	}

	if (Object == nullptr)
	{
		Object = NewObject<UPhysicsAsset>(Package, *ObjectName, RF_Public | RF_Standalone);

		FAssetRegistryModule::AssetCreated(Object);
	}

	UPhysicsAsset* NewPhysicsAsset = Cast<UPhysicsAsset>(Object);
	if (NewPhysicsAsset)
	{
		NewPhysicsAsset->MarkPackageDirty();

		FPhysAssetCreateParams NewBodyData;
		NewBodyData.LodIndex = LodIndex;
		FText CreationErrorMessage;
		bool bSuccess = FPhysicsAssetUtils::CreateFromSkeletalMesh(NewPhysicsAsset, SkeletalMesh, NewBodyData, CreationErrorMessage, bSetToMesh);
		if (!bSuccess)
		{
			UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "CreatePhysicsAsset failed: Couldn't create PhysicsAsset for the SkeletalMesh.");
			ObjectTools::DeleteObjects({ NewPhysicsAsset }, false);
			return nullptr;
		}
		else
		{
			RefreshSkelMeshOnPhysicsAssetChange(SkeletalMesh);
		}
	}

	return NewPhysicsAsset;
}

bool USkeletalMeshEditorSubsystem::IsPhysicsAssetCompatible(USkeletalMesh* TargetMesh, UPhysicsAsset* PhysicsAsset)
{
	if(!TargetMesh)
	{
		return false;
	}

	// If we have a physics asset, validate it. Otherwise the user is clearing the asset by passing a nullptr
	if (PhysicsAsset)
	{
		const FReferenceSkeleton& RefSkel = TargetMesh->GetRefSkeleton();

		// Check that all of the body setups correctly map to a skeletal mesh bone
		for (USkeletalBodySetup* Setup : PhysicsAsset->SkeletalBodySetups)
		{
			if (RefSkel.FindBoneIndex(Setup->BoneName) == INDEX_NONE)
			{
				UE_LOGF(LogSkeletalMeshEditorSubsystem, 
				       Error, 
				       "IsPhysicsAssetCompatible failed: bone %ls is not present for skeletal mesh %ls", 
				       *Setup->BoneName.ToString(), 
				       *TargetMesh->GetName());

				return false;
			}
		}

		// Check that all of the constraints match up to existing skeletal mesh bones.
		for (UPhysicsConstraintTemplate* Constraint : PhysicsAsset->ConstraintSetup)
		{
			int32 ConstraintBoneIndices[2] = {
				RefSkel.FindBoneIndex(Constraint->DefaultInstance.ConstraintBone1),
				RefSkel.FindBoneIndex(Constraint->DefaultInstance.ConstraintBone2)
			};



			if (ConstraintBoneIndices[0] == INDEX_NONE)
			{
				UE_LOGF(LogSkeletalMeshEditorSubsystem, 
				       Error, 
				       "IsPhysicsAssetCompatible failed: bone %ls is not present for skeletal mesh %ls", 
				       *Constraint->DefaultInstance.ConstraintBone1.ToString(), 
				       *TargetMesh->GetName());

				return false;
			}

			if (ConstraintBoneIndices[1] == INDEX_NONE)
			{
				UE_LOGF(LogSkeletalMeshEditorSubsystem, 
				       Error, 
				       "IsPhysicsAssetCompatible failed: bone %ls is not present for skeletal mesh %ls", 
				       *Constraint->DefaultInstance.ConstraintBone2.ToString(), 
				       *TargetMesh->GetName());

				return false;
			}
		}
	}

	return true;
}

bool USkeletalMeshEditorSubsystem::AssignPhysicsAsset(USkeletalMesh* TargetMesh, UPhysicsAsset* PhysicsAsset)
{
	if(!IsPhysicsAssetCompatible(TargetMesh, PhysicsAsset))
	{
		return false;
	}

	{
		FScopedTransaction Transaction(LOCTEXT("SetSkeletalMeshPhysicsAsset", "Set skeletal mesh physics asset"));
		TargetMesh->Modify();
		TargetMesh->SetPhysicsAsset(PhysicsAsset);
	}

	// Refresh all currently active skeletal mesh components that may have used the old setup
	RefreshSkelMeshOnPhysicsAssetChange(TargetMesh);

	return true;
}

bool USkeletalMeshEditorSubsystem::SetMorphTargetsToGeneratedByEngine(USkeletalMesh* SkeletalMesh, const TArray<FString>& OptionalNames)
{
	TArray<FString> MorphTargetNames = SkeletalMesh->K2_GetAllMorphTargetNames();
	TArray<FString> ToSetGeneratedByEngineMorphTargetNames;
	if (OptionalNames.IsEmpty())
	{
		ToSetGeneratedByEngineMorphTargetNames = MorphTargetNames;
	}
	else
	{
		for (const FString& OptionalMorphTargetName : OptionalNames)
		{
			if (MorphTargetNames.Contains(OptionalMorphTargetName))
			{
				ToSetGeneratedByEngineMorphTargetNames.Add(OptionalMorphTargetName);
			}
		}

	}
	if (ToSetGeneratedByEngineMorphTargetNames.IsEmpty())
	{
		return false;
	}
	bool bPreEditChangeCalled = false;
	auto DoPreEditChange = [&SkeletalMesh, &bPreEditChangeCalled]()
		{
			if (!bPreEditChangeCalled)
			{
				bPreEditChangeCalled = true;
				SkeletalMesh->PreEditChange(nullptr);
			}
		};
	const int32 LodCount = SkeletalMesh->GetLODNum();
	for (int32 LodIndex = 0; LodIndex < LodCount; ++LodIndex)
	{
		FSkeletalMeshLODInfo* LodInfo = SkeletalMesh->GetLODInfo(LodIndex);
		if (LodInfo)
		{
			for (const FString& MorphTargetNameStr : ToSetGeneratedByEngineMorphTargetNames)
			{
				constexpr bool bGeneratedByEngineTrue = true;
				if (LodInfo->ImportedMorphTargetSourceFilename.Contains(MorphTargetNameStr))
				{
					FMorphTargetImportedSourceFileInfo& MorphTargetImportedSourceFileInfo = LodInfo->ImportedMorphTargetSourceFilename.FindChecked(MorphTargetNameStr);
					if(!MorphTargetImportedSourceFileInfo.IsGeneratedByEngine())
					{
						DoPreEditChange();
						MorphTargetImportedSourceFileInfo.SetGeneratedByEngine(bGeneratedByEngineTrue);
					}
				}
				else
				{
					DoPreEditChange();
					LodInfo->ImportedMorphTargetSourceFilename.FindOrAdd(MorphTargetNameStr).SetGeneratedByEngine(bGeneratedByEngineTrue);
				}
			}
		}
	}
	if (bPreEditChangeCalled)
	{
		SkeletalMesh->PostEditChange();
	}
	return bPreEditChangeCalled;
}

void USkeletalMeshEditorSubsystem::SetMorphTargetsToGeneratedByEngineForAllSkeletalMesh(const TArray<FString>& OptionalNames, const TArray<FName>& OptionalPaths)
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	TArray<FAssetData> AssetDatas;
	FARFilter Filter;
	
	Filter.ClassPaths.Add(USkeletalMesh::StaticClass()->GetClassPathName());
	Filter.bRecursiveClasses = false;
	if (!OptionalPaths.IsEmpty())
	{
		Filter.bRecursivePaths = true;
		Filter.PackagePaths = OptionalPaths;
		Filter.bIncludeOnlyOnDiskAssets = false;
	}
	AssetRegistryModule.Get().GetAssets(Filter, AssetDatas);

	const int32 AssetCount = AssetDatas.Num();
	FText ProgressText = FText::Format(LOCTEXT("SetMorphTargetsToGeneratedByEngineForAllSkeletalMesh_SlowTask", "Iterating {0} skeletalmeshes..."), FText::AsNumber(AssetCount));
	FScopedSlowTask Progress(AssetCount, ProgressText);
	Progress.MakeDialog(true);
	int32 ModifiedSkeletalMeshCount = 0;
	for (const FAssetData& SkeletalMeshAsset : AssetDatas)
	{
		//Break if user ask to cancel the operation
		if (Progress.ShouldCancel())
		{
			break;
		}
		//SkeletalMeshAsset.FindTag()
		USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(SkeletalMeshAsset.GetAsset());
		if (SetMorphTargetsToGeneratedByEngine(SkeletalMesh, OptionalNames))
		{
			ModifiedSkeletalMeshCount++;
			ProgressText = FText::Format(LOCTEXT("SetMorphTargetsToGeneratedByEngineForAllSkeletalMesh_SlowTask_modified", "Iterating {0} skeletalmeshes... modified count {1}"), FText::AsNumber(AssetCount), FText::AsNumber(ModifiedSkeletalMeshCount));
		}
		Progress.EnterProgressFrame(1, ProgressText);
	}
}

void USkeletalMeshEditorSubsystem::GetMorphTargetsGeneratedByEngine(USkeletalMesh* TargetMesh, TArray<FName>& OutNames)
{
	constexpr int32 LodIndex = 0;
	if (FSkeletalMeshLODInfo* LodInfo = TargetMesh->GetLODInfo(LodIndex))
	{
		for (const TPair<FString, FMorphTargetImportedSourceFileInfo>& Info : LodInfo->ImportedMorphTargetSourceFilename)
		{
			if (Info.Value.IsGeneratedByEngine())
			{
				OutNames.Add(*(Info.Key));
			}
		}
	}
}

void USkeletalMeshEditorSubsystem::SetAllowCPUAccess(USkeletalMesh* TargetMesh, bool bAllowCPUAccess)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return;
	}

	if (TargetMesh == nullptr)
	{
		UE_LOGF(LogSkeletalMeshEditorSubsystem, Error, "SetAllowCPUAccess: The SkeletalMesh is null.");
		return;
	}

	TargetMesh->Modify();

	for (int32 i=0; i < TargetMesh->GetLODNum(); ++i)
	{
		TargetMesh->GetLODInfo(i)->bAllowCPUAccess = bAllowCPUAccess;
	}

	TargetMesh->PostEditChange();
}

void USkeletalMeshEditorSubsystem::GetSkeletonCurveMetaDataNames(const USkeleton* TargetSkeleton, TArray<FName>& OutNames, ESkelSubSysQueryCurvesMetatdataNamesFilter Filter)
{
	if (!TargetSkeleton)
	{
		return;
	}
	TargetSkeleton->ForEachCurveMetaData([&OutNames, &Filter](FName CurveName, const FCurveMetaData& CurveMetaData)
	{
		switch(Filter)
		{
			case ESkelSubSysQueryCurvesMetatdataNamesFilter::All:
			{
				OutNames.Add(CurveName);
				break;
			}
			case ESkelSubSysQueryCurvesMetatdataNamesFilter::MorphTargetOnly:
			{
				if (CurveMetaData.Type.bMorphtarget)
				{
					OutNames.Add(CurveName);
				}
				break;
			}
			case ESkelSubSysQueryCurvesMetatdataNamesFilter::MaterialOnly:
			{
				if (CurveMetaData.Type.bMaterial)
				{
					OutNames.Add(CurveName);
				}
				break;
			}
		}
	});
}

#undef LOCTEXT_NAMESPACE
