// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PersonaAssetEditorToolkit.h"
#include "IHasPersonaToolkit.h"

class ISkeletalMeshEditorBinding;

class ISkeletalMeshEditor : public FPersonaAssetEditorToolkit, public IHasPersonaToolkit
{
public:
	virtual TSharedPtr<ISkeletalMeshEditorBinding> GetBinding() = 0;
	virtual FSimpleMulticastDelegate& OnPreSaveAsset() = 0;
	virtual FSimpleMulticastDelegate& OnPreSaveAssetAs() = 0;

	virtual TSharedPtr<FUICommandInfo> GetResetBoneTransformsCommand() = 0;
	virtual TSharedPtr<FUICommandInfo> GetResetAllBonesTransformsCommand() = 0;
};

