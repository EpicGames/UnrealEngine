// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "ISkeletalMeshEditor.h"

DECLARE_LOG_CATEGORY_EXTERN(LogSkeletalMeshEditor, Log, All);

class ISkeletalMeshEditorModule : public IModuleInterface, public IHasMenuExtensibility, public IHasToolBarExtensibility
{
public:
	/** Creates a new skeleton editor instance */
	virtual TSharedRef<ISkeletalMeshEditor> CreateSkeletalMeshEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, class USkeletalMesh* InSkeletalMesh) = 0;

	/** Get all toolbar extenders */
	DECLARE_DELEGATE_RetVal_TwoParams(TSharedRef<FExtender>, FSkeletalMeshEditorToolbarExtender, const TSharedRef<FUICommandList> /*InCommandList*/, TSharedRef<ISkeletalMeshEditor> /*InSkeletalMeshEditor*/);
	virtual TArray<FSkeletalMeshEditorToolbarExtender>& GetAllSkeletalMeshEditorToolbarExtenders() = 0;

	/** Add callback after editor has been initialized */
	DECLARE_DELEGATE_OneParam(FOnSkeletalMeshEditorInitialized, TWeakPtr<ISkeletalMeshEditor>);
	virtual TArray<FOnSkeletalMeshEditorInitialized>& GetPostEditorInitDelegates() = 0;

	// Used by the EditorUISubsystem
	DECLARE_EVENT_OneParam(FExampleCharacterFXEditorModule, FOnRegisterLayoutExtensions, FLayoutExtender&);
	virtual FOnRegisterLayoutExtensions& OnRegisterLayoutExtensions() = 0;
};
