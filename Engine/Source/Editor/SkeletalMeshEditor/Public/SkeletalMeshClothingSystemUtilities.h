// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR
#include "CoreMinimal.h"

#define UE_API SKELETALMESHEDITOR_API

class USkeletalMesh;
class UClothingAssetBase;

/** Editor utilities for managing skeletal mesh clothing assets */
struct FSkeletalMeshClothingSystemUtilities
{
	/** Assign a clothing asset to a skeletal mesh section. Returns success.*/
	static UE_API bool AssignClothingToSection(USkeletalMesh* SkeletalMesh, UClothingAssetBase* ClothingAsset, int32 SkeletalMeshLodIndex, int32 SkeletalMeshSection, int32 ClothingLodIndex, FString* ErrorMessage);

	/** Remove the clothing from a skeletal mesh section. Returns success.*/
	static UE_API bool RemoveClothingFromSection(USkeletalMesh* SkeletalMesh, int32 SkeletalMeshLodIndex, int32 SkeletalMeshSection, FString* ErrorMessage);
};

#undef UE_API
#endif // WITH_EDITOR
