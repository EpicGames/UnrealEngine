// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ToolMenuContext.h"
#include "SkeletalMeshEditorContextMenuContext.generated.h"

UCLASS(MinimalAPI)
class USkeletalMeshEditorContextMenuContext : public UObject
{
	GENERATED_BODY()

public:
	// The LOD index that was clicked
	int32 LodIndex = INDEX_NONE;

	// The section index that was clicked
	int32 SectionIndex = INDEX_NONE;
};
