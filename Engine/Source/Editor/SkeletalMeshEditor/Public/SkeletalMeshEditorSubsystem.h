// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "EditorSubsystem.h"
#include "Engine/EngineTypes.h"

#include "SkeletalMeshEditorSubsystem.generated.h"

#define UE_API SKELETALMESHEDITOR_API

DECLARE_LOG_CATEGORY_EXTERN(LogSkeletalMeshEditorSubsystem, Log, All);

class USkeletalMesh;
class USkeleton;
class UMaterialInterface;

/**
 * Blueprint enum to query skeleton curves
 */
UENUM(BlueprintType)
enum class ESkelSubSysQueryCurvesMetatdataNamesFilter : uint8
{
	/** Query all metadata curves */
	All,
	/** Query only morph target metadata curves */
	MorphTargetOnly,
	/** Query only material metadata curves */
	MaterialOnly
};

/**
* USkeletalMeshEditorSubsystem
* Subsystem for exposing skeletal mesh functionality to scripts
*/
UCLASS(MinimalAPI)
class USkeletalMeshEditorSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:
	UE_API USkeletalMeshEditorSubsystem();

	/** Regenerate LODs of the mesh
	 *
	 * @param SkeletalMesh	The mesh that will regenerate LOD
	 * @param NewLODCount	Set valid value (>0) if you want to change LOD count.
	 *						Otherwise, it will use the current LOD and regenerate
	 * @param bRegenerateEvenIfImported	If this is true, it only regenerate even if this LOD was imported before
	 *									If false, it will regenerate for only previously auto generated ones
	 * @param bGenerateBaseLOD If this is true and there is some reduction data, the base LOD will be reduce according to the settings
	 * @return	true if succeed. If mesh reduction is not available this will return false.
	 */
	UFUNCTION(BlueprintCallable, Category = "Skeletal Mesh Utilities", meta = (ScriptMethod))
	static UE_API bool RegenerateLOD(USkeletalMesh* SkeletalMesh, int32 NewLODCount = 0, bool bRegenerateEvenIfImported = false, bool bGenerateBaseLOD = false);

	/** Get number of mesh vertices for an LOD of a Skeletal Mesh
	 *
	 * @param SkeletalMesh		Mesh to get number of vertices from.
	 * @param LODIndex			Index of the mesh LOD.
	 * @return Number of vertices. Returns 0 if invalid mesh or LOD index.
	 */
	UFUNCTION(BlueprintPure, Category = "Skeletal Mesh Utilities")
	UE_API int32 GetNumVerts(USkeletalMesh* SkeletalMesh, int32 LODIndex);

	/** Get number of sections for a LOD of a Skeletal Mesh
	 *
	 * @param SkeletalMesh		Mesh to get number of vertices from.
	 * @param LODIndex			Index of the mesh LOD.
	 * @return Number of sections. Returns INDEX_NONE if invalid mesh or LOD index.
	 */
	UFUNCTION(BlueprintPure, Category = "Skeletal Mesh Utilities")
	UE_API int32 GetNumSections(USkeletalMesh* SkeletalMesh, int32 LODIndex);

	/** Get bRecomputeTangent from a section of a LOD of a Skeletal Mesh
	 *
	 * @param SkeletalMesh			Mesh to get number of vertices from.
	 * @param LodIndex				Index of the mesh LOD.
	 * @param SectionIndex			Index of the LOD section.
	 * @param bOutRecomputeTangent	The function will set the bRecomputeTangent used by the section
	 * @return false if invalid mesh or LOD index or section index.
	 */
	UFUNCTION(BlueprintPure, Category = "Skeletal Mesh Utilities")
	UE_API bool GetSectionRecomputeTangent(const USkeletalMesh* SkeletalMesh, const int32 LODIndex, const int32 SectionIndex, bool& bOutRecomputeTangent);

	/** Set bRecomputeTangent for a section of a LOD of a Skeletal Mesh.
	 *
	 * @param SkeletalMesh			Mesh to get number of vertices from.
	 * @param LodIndex				Index of the mesh LOD.
	 * @param SectionIndex			Index of the LOD section.
	 * @param bRecomputeTangent	The function will set the bRecomputeTangent used by the section
	 * @return false if invalid mesh or LOD index or section index.
	 */
	UFUNCTION(BlueprintCallable, Category = "Skeletal Mesh Utilities")
	UE_API bool SetSectionRecomputeTangent(USkeletalMesh* SkeletalMesh, const int32 LODIndex, const int32 SectionIndex, const bool bRecomputeTangent);

	/** Get RecomputeTangentsVertexMaskChannel from a section of a LOD of a Skeletal Mesh
	 *
	 * @param SkeletalMesh			Mesh to get number of vertices from.
	 * @param LodIndex				Index of the mesh LOD.
	 * @param SectionIndex			Index of the LOD section.
	 * @param OutRecomputeTangentsVertexMaskChannel	The function will set the RecomputeTangentsVertexMaskChannel used by the section
	 * @return false if invalid mesh or LOD index or section index.
	 */
	UFUNCTION(BlueprintPure, Category = "Skeletal Mesh Utilities")
	UE_API bool GetSectionRecomputeTangentsVertexMaskChannel(const USkeletalMesh* SkeletalMesh, const int32 LODIndex, const int32 SectionIndex, uint8& OutRecomputeTangentsVertexMaskChannel);

	/** Set RecomputeTangentsVertexMaskChannel for a section of a LOD of a Skeletal Mesh.
	 *
	 * @param SkeletalMesh			Mesh to get number of vertices from.
	 * @param LodIndex				Index of the mesh LOD.
	 * @param SectionIndex			Index of the LOD section.
	 * @param RecomputeTangentsVertexMaskChannel	The function will set the RecomputeTangentsVertexMaskChannel used by the section
	 * @return false if invalid mesh or LOD index or section index.
	 */
	UFUNCTION(BlueprintCallable, Category = "Skeletal Mesh Utilities")
	UE_API bool SetSectionRecomputeTangentsVertexMaskChannel(USkeletalMesh* SkeletalMesh, const int32 LODIndex, const int32 SectionIndex, const uint8 RecomputeTangentsVertexMaskChannel);

	/** Get bCastShadow from a section of a LOD of a Skeletal Mesh
	 *
	 * @param SkeletalMesh			Mesh to get number of vertices from.
	 * @param LodIndex				Index of the mesh LOD.
	 * @param SectionIndex			Index of the LOD section.
	 * @param bOutCastShadow	The function will set the bCastShadow used by the section
	 * @return false if invalid mesh or LOD index or section index.
	 */
	UFUNCTION(BlueprintPure, Category = "Skeletal Mesh Utilities")
	UE_API bool GetSectionCastShadow(const USkeletalMesh* SkeletalMesh, const int32 LODIndex, const int32 SectionIndex, bool& bOutCastShadow);

	/** Set bCastShadow for a section of a LOD of a Skeletal Mesh.
	 *
	 * @param SkeletalMesh			Mesh to get number of vertices from.
	 * @param LodIndex				Index of the mesh LOD.
	 * @param SectionIndex			Index of the LOD section.
	 * @param bCastShadow	The function will set the bCastShadow used by the section
	 * @return false if invalid mesh or LOD index or section index.
	 */
	UFUNCTION(BlueprintCallable, Category = "Skeletal Mesh Utilities")
	UE_API bool SetSectionCastShadow(USkeletalMesh* SkeletalMesh, const int32 LODIndex, const int32 SectionIndex, const bool bCastShadow);

	/** Get bVisibleInRayTracing from a section of a LOD of a Skeletal Mesh
	 *
	 * @param SkeletalMesh			Mesh to get number of vertices from.
	 * @param LodIndex				Index of the mesh LOD.
	 * @param SectionIndex			Index of the LOD section.
	 * @param bOutVisibleInRayTracing	The function will set the bVisibleInRayTracing used by the section
	 * @return false if invalid mesh or LOD index or section index.
	 */
	UFUNCTION(BlueprintPure, Category = "Skeletal Mesh Utilities")
	UE_API bool GetSectionVisibleInRayTracing(const USkeletalMesh* SkeletalMesh, const int32 LODIndex, const int32 SectionIndex, bool& bOutVisibleInRayTracing);

	/** Set bVisibleInRayTracing for a section of a LOD of a Skeletal Mesh.
	 *
	 * @param SkeletalMesh			Mesh to get number of vertices from.
	 * @param LodIndex				Index of the mesh LOD.
	 * @param SectionIndex			Index of the LOD section.
	 * @param bVisibleInRayTracing	The function will set the bVisibleInRayTracing used by the section
	 * @return false if invalid mesh or LOD index or section index.
	 */
	UFUNCTION(BlueprintCallable, Category = "Skeletal Mesh Utilities")
	UE_API bool SetSectionVisibleInRayTracing(USkeletalMesh* SkeletalMesh, const int32 LODIndex, const int32 SectionIndex, const bool bVisibleInRayTracing);

	/** Get the overlay material for a material slot in a Skeletal Mesh.
	 *
	 * @param SkeletalMesh			Mesh to get material overlay from.
	 * @param SlotIndex				Index of the skeletal mesh material slot.
	 * @return the material interface or null if there is no overlay material.
	 */
	UFUNCTION(BlueprintPure, Category = "Skeletal Mesh Utilities")
	UE_API UMaterialInterface* GetMaterialSlotOverlayMaterial(const USkeletalMesh* SkeletalMesh, const int32 SlotIndex);

	/** Set the overlay material for a material slot in a Skeletal Mesh.
	 *
	 * @param SkeletalMesh			Mesh to set material overlay to.
	 * @param SlotIndex				Index of the skeletal mesh material slot.
	 * @param NewSectionOverlayMaterial	The function will set the OverlayMaterial used by the slot
	 * @return false if invalid mesh or material slot index is invalid.
	 */
	UFUNCTION(BlueprintCallable, Category = "Skeletal Mesh Utilities")
	UE_API bool SetMaterialSlotOverlayMaterial(USkeletalMesh* SkeletalMesh, const int32 SlotIndex, UMaterialInterface* NewSectionOverlayMaterial);

	/** Get the overlay material for a Skeletal Mesh.
	 *
	 * @param SkeletalMesh			Mesh to get the material overlay from.
	 * @return the material interface or null if there is no material.
	 */
	UFUNCTION(BlueprintPure, Category = "Skeletal Mesh Utilities")
	UE_API UMaterialInterface* GetSkeletalMeshOverlayMaterial(const USkeletalMesh* SkeletalMesh);

	/** Set the overlay material for a Skeletal Mesh.
	 *
	 * @param SkeletalMesh					Mesh to set material overlay to.
	 * @param NewOverlayMaterial		the OverlayMaterial we set for the skeletal mesh
	 * @return false if invalid mesh.
	 */
	UFUNCTION(BlueprintCallable, Category = "Skeletal Mesh Utilities")
	UE_API bool SetSkeletalMeshOverlayMaterial(USkeletalMesh* SkeletalMesh, UMaterialInterface* NewOverlayMaterial);

	/**
	 * Gets the material slot used for a specific LOD section.
	 * @param	SkeletalMesh		SkeletalMesh to get the material index from.
	 * @param	LODIndex			Index of the StaticMesh LOD.
	 * @param	SectionIndex		Index of the StaticMesh Section.
	 * @return  MaterialSlotIndex	Index of the material slot used by the section or INDEX_NONE in case of error.
	 */
	UFUNCTION(BlueprintCallable, Category = "Skeletal Mesh Utilities")
	UE_API int32 GetLODMaterialSlot(USkeletalMesh* SkeletalMesh, int32 LODIndex, int32 SectionIndex);

	/** Rename a socket within a skeleton
	 * @param SkeletalMesh	The mesh inside which we are renaming a socket
	 * @param OldName       The old name of the socket
	 * @param NewName		The new name of the socket
	 * @return true if the renaming succeeded.
	 */
	UFUNCTION(BlueprintCallable, Category = "Skeletal Mesh Utilities", meta = (ScriptMethod))
	static UE_API bool RenameSocket(USkeletalMesh* SkeletalMesh, FName OldName, FName NewName);

	/** Remove a socket from a skeletal mesh
	 * @param SkeletalMesh	The mesh from which we are removing a socket
	 * @param SocketName    The name of the socket to remove
	 * @return true if the removal succeeded.
	 */
	UFUNCTION(BlueprintCallable, Category = "Skeletal Mesh Utilities", meta = (ScriptMethod))
	static UE_API bool RemoveSocket(USkeletalMesh* SkeletalMesh, FName SocketName);

	/** Get the parent bone name of a bone in a skeletal mesh.
	 * @param SkeletalMesh  The mesh to query.
	 * @param BoneName      The name of the bone to query.
	 * @return The name of the parent bone, or NAME_None if the bone is the root or does not exist.
	 */
	UFUNCTION(BlueprintCallable, Category = "Skeletal Mesh Utilities", meta = (ScriptMethod))
	static UE_API FName GetBoneParent(const USkeletalMesh* SkeletalMesh, FName BoneName);

	/** Get the direct children of a bone in a skeletal mesh.
	 * @param SkeletalMesh  The mesh to query.
	 * @param BoneName      The name of the bone to query.
	 * @return An array of bone names that are direct children of the given bone.
	 */
	UFUNCTION(BlueprintCallable, Category = "Skeletal Mesh Utilities", meta = (ScriptMethod))
	static UE_API TArray<FName> GetBoneChildren(const USkeletalMesh* SkeletalMesh, FName BoneName);

	/**
	 * Retrieve the number of LOD contain in the specified skeletal mesh.
	 *
	 * @param SkeletalMesh: The static mesh we import or re-import a LOD.
	 *
	 * @return The LOD number.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | SkeletalMesh")
	static UE_API int32 GetLODCount(USkeletalMesh* SkeletalMesh);

	/**
	 * Import or re-import a LOD into the specified base mesh. If the LOD do not exist it will import it and add it to the base static mesh. If the LOD already exist it will re-import the specified LOD.
	 *
	 * @param BaseSkeletalMesh: The static mesh we import or re-import a LOD.
	 * @param LODIndex: The index of the LOD to import or re-import. Valid value should be between 0 and the base skeletal mesh LOD number. Invalid value will return INDEX_NONE
	 * @param SourceFilename: The fbx source filename. If we are re-importing an existing LOD, it can be empty in this case it will use the last import file. Otherwise it must be an existing fbx file.
	 *
	 * @return The index of the LOD that was imported or re-imported. Will return INDEX_NONE if anything goes bad.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | SkeletalMesh")
	static UE_API int32 ImportLOD(USkeletalMesh* BaseMesh, const int32 LODIndex, const FString& SourceFilename);

	/** Remove all the specified LODs. This function will remove all the valid LODs in the list.
	 * Valid LOD is any LOD greater then 0 that exist in the skeletalmesh. We cannot remove the base LOD 0.
	 *
	 * @param SkeletalMesh	The mesh inside which we are renaming a socket
	 * @param ToRemoveLODs	The LODs we need to remove
	 * @return true if the successfully remove all the LODs. False otherwise, but evedn if it return false it
	 * will have removed all valid LODs.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | SkeletalMesh", meta=(ScriptName="RemoveLods;RemoveLODs"))
	static UE_API bool RemoveLODs(USkeletalMesh* BaseMesh, const TArray<int32> ToRemoveLODs);

	/**
	 * This function will strip all triangle in the specified LOD that don't have any UV area pointing on a black pixel in the TextureMask.
	 * We use the UVChannel 0 to find the pixels in the texture.
	 *
	 * @Param SkeletalMesh: The skeletalmesh we want to optimize
	 * @Param LODIndex: The LOD we want to optimize
	 * @Param TextureMask: The texture containing the stripping mask. non black pixel strip triangle, black pixel keep them.
	 * @Param Threshold: The threshold we want when comparing the texture value with zero.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | SkeletalMesh")
	static UE_API bool StripLODGeometry(USkeletalMesh* SkeletalMesh, const int32 LODIndex, UTexture2D* TextureMask, const float Threshold);
	
	/**
	 * Re-import the specified skeletal mesh and all the custom LODs.
	 *
	 * @param SkeletalMesh: is the skeletal mesh we import or re-import a LOD.
	 *
	 * @return true if re-import works, false otherwise see log for explanation.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | SkeletalMesh", meta=(ScriptName="ReimportAllCustomLods;ReimportAllCustomLODs"))
	static UE_API bool ReimportAllCustomLODs(USkeletalMesh* SkeletalMesh);

	/**
	 * Copy the build options with the specified LOD build settings.
	 * @param SkeletalMesh - Mesh to process.
	 * @param LodIndex - The LOD we get the reduction settings.
	 * @param OutBuildOptions - The build settings where we copy the build options.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | SkeletalMesh")
	static UE_API void GetLodBuildSettings(const USkeletalMesh* SkeletalMesh, const int32 LodIndex, FSkeletalMeshBuildSettings& OutBuildOptions);

	/**
	 * Set the LOD build options for the specified LOD index.
	 * @param SkeletalMesh - Mesh to process.
	 * @param LodIndex - The LOD we will apply the build settings.
	 * @param BuildOptions - The build settings we want to apply to the LOD.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | SkeletalMesh")
	static UE_API void SetLodBuildSettings(USkeletalMesh* SkeletalMesh, const int32 LodIndex, const FSkeletalMeshBuildSettings& BuildOptions);

	/**
	 * This function creates a PhysicsAsset for the given SkeletalMesh with the same settings as if it were created through FBX import
	 *
	 * @Param SkeletalMesh: The SkeletalMesh we want to create the PhysicsAsset for
	 * @Param bSetToMesh: If true the created physics asset will be used by the skeletal mesh.
	 * @Param LodIndex: Use the specified LOD to create the physics asset.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | SkeletalMesh")
	static UE_API UPhysicsAsset* CreatePhysicsAsset(USkeletalMesh* SkeletalMesh, bool bSetToMesh = true, int32 LodIndex = 0);

	/**
	 * Checks whether a physics asset is compatible with the given SkeletalMesh
	 *
	 * @param TargetMesh The mesh to test for compatibility
	 * @param PhysicsAsset The PhysicsAsset to test for compatibility
	 *
	 * @return Whether the physics asset is compatible with the target SkeletalMesh
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | SkeletalMesh")
	static UE_API bool IsPhysicsAssetCompatible(USkeletalMesh* TargetMesh, UPhysicsAsset* PhysicsAsset);

	/**
	 * Assigns a PhysicsAsset to the given SkeletalMesh if it is compatible. Passing
	 * nullptr / None as the physics asset will always succeed and will clear the
	 * physics asset assignment for the target SkeletalMesh
	 *
	 * @param TargetMesh The mesh to attempt to assign the PhysicsAsset to
	 * @param PhysicsAsset The physics asset to assign to the provided mesh (or nullptr/None)
	 *
	 * @return Whether the physics asset was successfully assigned to the mesh
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | SkeletalMesh")
	static UE_API bool AssignPhysicsAsset(USkeletalMesh* TargetMesh, UPhysicsAsset* PhysicsAsset);

	/**
	 * Set morph targets has generated by engine.
	 * 
	 * @param TargetMesh The mesh to convert morph targets to generated by engine.
	 * @param OptionalNames The names of morph targets to convert. If the array is empty all morph will be converted.
	 * @return Return true if the target mesh was modified, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | SkeletalMesh")
	static UE_API bool SetMorphTargetsToGeneratedByEngine(USkeletalMesh* TargetMesh, const TArray<FString>& OptionalNames);

	/**
	 * Set morph targets has generated by engine for all skeletalmesh in the content.
	 *
	 * @param OptionalNames The names of morph targets to convert. If the array is empty all morph will be converted.
	 * @param OptionalPaths The paths list use to filter the research of the skeletal meshes. If the array is empty we will find all skeletal meshes.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | SkeletalMesh")
	static UE_API void SetMorphTargetsToGeneratedByEngineForAllSkeletalMesh(const TArray<FString>& OptionalNames, const TArray<FName>& OptionalPaths);

	/**
	* Get morph targets marked as generated by engine.
	* 
	* @param TargetMesh The mesh to get morph targets.
	* @param OutNames - Receive the names of morph targets marked as generated by engine
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | SkeletalMesh")
	static UE_API void GetMorphTargetsGeneratedByEngine(USkeletalMesh* TargetMesh, TArray<FName>& OutNames);

	/** Sets Skeletal Mesh Flag bAllowCPUAccess on all LOD levels. */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | SkeletalMesh")
	static UE_API void SetAllowCPUAccess(USkeletalMesh* TargetMesh, bool bAllowCPUAccess);

	/**
	 * Retrieve all curve metadata name from the skeleton.
	 * 
	 * @param TargetSkeleton The skeleton we query the data from.
	 * @param OutNames - Receive all the name of the curves that has metadata in the skeleton.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Skeleton")
	static UE_API void GetSkeletonCurveMetaDataNames(const USkeleton* TargetSkeleton, TArray<FName>& OutNames, ESkelSubSysQueryCurvesMetatdataNamesFilter Filter);
};

#undef UE_API
