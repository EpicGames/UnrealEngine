// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class SkeletalMeshEditor : ModuleRules
{
	public SkeletalMeshEditor(ReadOnlyTargetRules Target) : base(Target)
	{
        PublicDependencyModuleNames.AddRange(
            new string[] {
                "Persona",
            }
        );

        PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
                "InputCore",
				"Slate",
				"SlateCore",
				"EditorFramework",
				"EditorInteractiveToolsFramework",
                "UnrealEd",
                "SkeletonEditor",
                "Kismet",
                "KismetWidgets",
                "ActorPickerMode",
                "SceneDepthPickerMode",
                "MainFrame",
                "DesktopPlatform",
                "PropertyEditor",
                "RHI",
                "ClothingSystemRuntimeCommon",
                "ClothingSystemEditorInterface",
				"ClothingSystemRuntimeInterface",
				"SkeletalMeshUtilitiesCommon",
				"ToolMenus",
				"EditorSubsystem",
				"StatusBar",
				"PhysicsUtilities",
				"InterchangeCore",
				"InterchangeEngine",
				"WorkspaceMenuStructure",
				"TypedElementRuntime"
			}
		);
	}
}
